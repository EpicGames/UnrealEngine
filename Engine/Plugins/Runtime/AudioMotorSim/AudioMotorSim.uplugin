{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Audio Motor Sim",
	"Description": "Compositional method for simulating audio for vehicles.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "AudioMotorSim",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "AudioMotorSimStandardComponents",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AudioMotorSimDebug",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetConfigurationDenyList": [ "Shipping" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "SlateIM",
			"Enabled": true,
			"TargetConfigurationDenyList": [ "Shipping" ]
		}
	]
}