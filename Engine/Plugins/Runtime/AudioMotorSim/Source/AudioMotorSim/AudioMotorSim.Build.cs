// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMotorSim : ModuleRules
{
	public AudioMotorSim(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
		new string[]
			{
				"Core",
				"CoreUObject",
				"Engine"
			}
		);
	}
}
