// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMotorSimModule.h"
#include "AudioMotorSimLogs.h"
#include "Modules/ModuleManager.h"

class FAudioMotorSimModule : public IModuleInterface
{
public:
	virtual void StartupModule() override {};
	virtual void ShutdownModule() override {};
};

IMPLEMENT_MODULE(FAudioMotorSimModule, AudioMotorSim);

DEFINE_LOG_CATEGORY(LogAudioMotorSim)