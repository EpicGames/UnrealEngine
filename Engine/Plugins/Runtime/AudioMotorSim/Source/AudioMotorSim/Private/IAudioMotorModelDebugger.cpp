// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioMotorModelDebugger.h"

const FLazyName AudioMotorModelDebugger::DebuggerModularFeatureName = TEXT("AudioMotorModelDebugger");
