// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/ActorComponent.h"
#include "AudioMotorSimConfigData.h"
#include "AudioMotorSimTypes.h"
#include "StructUtils/InstancedStruct.h"

#include "AudioMotorModelComponent.generated.h"

#define UE_API AUDIOMOTORSIM_API

class IAudioMotorSim;
class IAudioMotorSimOutput;

USTRUCT(BlueprintType)
struct FMotorSimEntry
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "MotorSim")
	TScriptInterface<IAudioMotorSim> Sim;
	
	UPROPERTY(BlueprintReadOnly, Category = "MotorSim")
	int32 SortOrder = 0;
};
	
UCLASS(MinimalAPI, ClassGroup = AudioMotorSim, Blueprintable, meta = (BlueprintSpawnableComponent))
class UAudioMotorModelComponent : public UActorComponent
{
	GENERATED_BODY()
public:
	virtual void BeginPlay() override;
	
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Model")
	TArray<FMotorSimEntry> SimComponents;

	UPROPERTY(Transient)
	TArray<TScriptInterface<IAudioMotorSimOutput>> AudioComponents;

	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API virtual void Update(const FAudioMotorSimInputContext& Input);
	
	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API virtual void Reset();

	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API virtual void StartOutput();
	
	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API virtual void StopOutput();

	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API void AddMotorAudioComponent(TScriptInterface<IAudioMotorSimOutput> InComponent);
	
	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API void RemoveMotorAudioComponent(TScriptInterface<IAudioMotorSimOutput> InComponent);
	
	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API void AddMotorSimComponent(TScriptInterface<IAudioMotorSim> InComponent, const int32 SortOrder = 0);
	
	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API void RemoveMotorSimComponent(TScriptInterface<IAudioMotorSim> InComponent);
	
	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API void RemoveAllMotorSimComponents();

	UFUNCTION(BlueprintCallable, Category = MotorModel)
	UE_API void ConfigureMotorSimComponents(const TArray<FInstancedStruct>& InConfigData);
	
	UFUNCTION(BlueprintPure, Category = State)
	float GetRpm() const { return CachedRuntimeContext.Rpm; }

	UFUNCTION(BlueprintPure, Category = State)
	int32 GetGear() const { return CachedRuntimeContext.Gear; }
	
	UFUNCTION(BlueprintPure, Category = State)
	FAudioMotorSimRuntimeContext GetRuntimeInfo() const { return CachedRuntimeContext; }

	UFUNCTION(BlueprintPure, Category = State)
	const FAudioMotorSimInputContext& GetCachedInputData() const { return CachedInputContext; }

	template<typename T>
	TObjectPtr<T> GetMotorSimOfType()
	{
		static_assert(TIsDerivedFrom<T, IAudioMotorSim>::Value, "Type must be derived from IAudioMotorSim");
		
		for(auto SimIt = SimComponents.CreateIterator(); SimIt; ++SimIt)
		{
			if(SimIt->Sim.GetObject())
			{
				if(TObjectPtr<T> CastedSimComponent = Cast<T>(SimIt->Sim.GetObject()))
				{
					return CastedSimComponent;
				}
			}
		}
		
		return nullptr;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_API void DebugPrintRuntimeInfo() const;
	UE_API void DebugPrintInputContext() const;
#endif
	
private:
	FAudioMotorSimRuntimeContext CachedRuntimeContext;
	
	FAudioMotorSimInputContext CachedInputContext;
	
};

#undef UE_API
