// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioMotorSimTypes.generated.h"

// collection of properties to be filled out by the vehicle in order to update the motor sim
USTRUCT(BlueprintType)
struct FAudioMotorSimInputContext
{
	GENERATED_BODY()

	// Time in Seonds
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UpdateContext")
	float DeltaTime = 0.f;

	// Current speed of the vehicle in any direction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Velocity")
	float Speed = 0.f;
	
	// Current speed of the vehicle relative to its forward direction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Velocity")
	float ForwardSpeed = 0.f;

	// absolute value of the speed of the vehicle relative to its right direction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Velocity")
	float SideSpeed = 0.f;
	
	// Current speed of the vehicle along the z-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Velocity")
	float UpSpeed = 0.f;
	
	// normalized input representing the player wanting to accelerate [-1, 1]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	float Throttle = 0.f;
	
	// normalized input representing the player wanting to slow down [0, 1]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
    float Brake = 0.f;
	
	// scaling to apply to any behaviors that model surface friction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	float SurfaceFrictionModifier = 1.f;

	// scaling to apply to any behaviors that model internal motor friction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	float MotorFrictionModifier = 1.f;
	
	// normalized input representing additional thrust beyond normal driving behaviors
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	float Boost = 0.f;
	
	// whether a player can drive the vehicle right now
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	bool bDriving = false;
	
	// whether the vehicle is firmly on the ground
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	bool bGrounded = false;
	
	// whether the motor can freely shift gears
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	bool bCanShift = false;

	// when true, signals that the gears are temporarily disconnected from the motor
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	bool bClutchEngaged = false;
};

// properties that represent the current state of the motor sim, and persist between updates
USTRUCT(BlueprintType)
struct FAudioMotorSimRuntimeContext
{
	GENERATED_BODY()
		
	// true while the motor is performing a gear shift
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	bool bShifting = false;
		
	// which gear the motor is in, if it uses gears
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	int32 Gear = 0;
	
	// the virtual speed calculated by the Motor Physics Simulation
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "State", meta = (ForceUnits = "km/h"))
    float VirtualSpeedKmh = 0.f;
	
	// normalized RPM [0-1] of the motor
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	float Rpm = 0.f;
	
	// volume to set on the output component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	float Volume = 1.f;
	
	// pitch to set on the output component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "State")
	float Pitch = 1.f;
};

namespace AudioMotorSim
{
	inline float CmsToKmh(const float InSpeed) { return InSpeed * 0.036f; }
	inline float KmhToCms(const float InSpeed) { return InSpeed / 0.036f; }
}
