// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/NameTypes.h"

namespace AudioMotorModelDebugger
{
	extern const AUDIOMOTORSIM_API FLazyName DebuggerModularFeatureName;
}

class IAudioMotorModelDebugger : public IModularFeature
{
public:
	virtual ~IAudioMotorModelDebugger() = default;
	
	virtual void RegisterComponentWithDebugger(class UAudioMotorModelComponent* MotorModelComponent) = 0;
	virtual void SendAdditionalDebugData(UObject* Object, const FInstancedStruct& InData) = 0;
};
