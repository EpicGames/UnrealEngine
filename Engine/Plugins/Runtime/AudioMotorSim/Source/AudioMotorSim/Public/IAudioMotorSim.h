// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioMotorSimTypes.h"
#include "Components/ActorComponent.h"
#include "IAudioMotorModelDebugger.h"
#include "StructUtils/InstancedStruct.h"

#include "IAudioMotorSim.generated.h"

#define UE_API AUDIOMOTORSIM_API

class IAudioMotorModelDebugger;
struct FAudioMotorSimConfigData;
struct FAudioMotorSimInputContext;
struct FAudioMotorSimRuntimeContext;

USTRUCT()
struct FAudioMotorSimDebugDataBase
{
	GENERATED_BODY()
	
	TMap<FName, double> ParameterValues;
};

UINTERFACE(MinimalAPI, BlueprintType, NotBlueprintable)
class UAudioMotorSim : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IAudioMotorSim
{
	GENERATED_IINTERFACE_BODY()

public:
	virtual void Init() = 0;
	
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) = 0;

	// Use to reset any state that might be desired. Will be called automatically if the entire MotorSim is Reset, or call it manually
	UFUNCTION(BlueprintCallable, Category = "AudioMotorSim")
	virtual void Reset() {}

	UFUNCTION(BlueprintCallable, Category = "AudioMotorSim")
	virtual bool GetEnabled() { return false; }

	UFUNCTION(BlueprintCallable, Category = "AudioMotorSim")
	virtual void ConfigMotorSim(const FInstancedStruct& InConfigData) {};
};

UCLASS(MinimalAPI, Abstract, Blueprintable, EditInlineNew, Category = "AudioMotorSim", hideCategories = (Activation, AssetUserData, ComponentReplication, Cooking, Navigation, Replication, Tags, Variable), meta=(BlueprintSpawnableComponent))
class UAudioMotorSimComponent : public UActorComponent, public IAudioMotorSim
{
	GENERATED_BODY()

public:
	UE_API UAudioMotorSimComponent(const FObjectInitializer& ObjectInitializer);
	
	UE_API virtual void BeginPlay() override;

	UE_API virtual void Init() override;

	UE_API virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;

	UE_API virtual void Reset() override;

	virtual bool GetEnabled() override { return bEnabled; }
	
	/* Called every tick that this component is being updated. Use "Set Members in Struct" to update values for future components in the chain. The return value does nothing.
	* @param Input			Holds values which are not saved between update frames which represent input to the simulation
	* @param RuntimeInfo	Holds values which are saved between update frames to represent the output or state of the simulation
	* @return				Vestigial, does nothing.
	*/
	UFUNCTION(BlueprintImplementableEvent, Category = "AudioMotorSim", DisplayName = "OnUpdate")
	UE_API bool BP_Update(UPARAM(ref) FAudioMotorSimInputContext& Input, UPARAM(ref) FAudioMotorSimRuntimeContext& RuntimeInfo);
	
	// Called when something Resets this component
	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, Category="AudioMotorSim", DisplayName = "OnReset")
	UE_API void BP_Reset();
	
	// controls whether this will run its update function
	UFUNCTION(BlueprintCallable, Category="AudioMotorSim")
	UE_API void SetEnabled(bool bNewEnabled);

	UE_API virtual void ConfigMotorSim(const FInstancedStruct& InConfigData) override;

#if WITH_EDITORONLY_DATA
	// Input data after running this component
	UPROPERTY(BlueprintReadOnly, Category = "DebugInfo")
	FAudioMotorSimInputContext CachedInput;

	// runtime info after running this component
	UPROPERTY(BlueprintReadOnly, Category = "DebugInfo")
	FAudioMotorSimRuntimeContext CachedRuntimeInfo;

	UE_API virtual void GetCachedData(FAudioMotorSimInputContext& OutInput, FAudioMotorSimRuntimeContext& OutRuntimeInfo);
#endif

	// will only update if enabled
    UPROPERTY(BlueprintReadOnly, Category="AudioMotorSim")
    bool bEnabled = true;

protected:
	
#if !UE_BUILD_SHIPPING
	UE_API bool ShouldCollectDebugData() const;
	FInstancedStruct CachedDebugData;
#endif
	
private:
	
#if !UE_BUILD_SHIPPING
	void SendCachedDebugData();
	TArray<IAudioMotorModelDebugger*> MotorModelDebuggers;
	bool bCollectDebugData = false;
#endif
	
	bool bUpdateImplemented = true;
};

#undef UE_API
