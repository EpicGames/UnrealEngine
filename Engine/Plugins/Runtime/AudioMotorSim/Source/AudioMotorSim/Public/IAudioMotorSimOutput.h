// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Interface.h"
#include "IAudioMotorSimOutput.generated.h"

struct FAudioMotorSimInputContext;
struct FAudioMotorSimRuntimeContext;

UINTERFACE(MinimalAPI)
class UAudioMotorSimOutput : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IAudioMotorSimOutput
{
	GENERATED_IINTERFACE_BODY()
	
public:
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) = 0;

	virtual void StartOutput() = 0;
	virtual void StopOutput() = 0;
};
