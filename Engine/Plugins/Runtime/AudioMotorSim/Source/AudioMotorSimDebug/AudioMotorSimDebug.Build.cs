// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMotorSimDebug : ModuleRules
{
    public AudioMotorSimDebug(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
        
        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "AudioMotorSim",
	            "Core",
                "CoreUObject",
                "Engine",
                "SlateCore",
                "SlateIM"
            }
			
        );
        
        if (Target.Type == TargetType.Editor)
        {
	        PrivateDependencyModuleNames.AddRange(
		        new string[] {
			        "UnrealEd",
		        }
	        );
        }
    }
}