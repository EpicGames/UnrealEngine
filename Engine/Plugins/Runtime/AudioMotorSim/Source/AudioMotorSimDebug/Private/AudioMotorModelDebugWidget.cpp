// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMotorModelDebugWidget.h"

#include "AudioMotorModelComponent.h"
#include "AudioMotorModelDebuggerPropertyUI.h"
#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "IAudioMotorSim.h"
#include "MotorSimComponentDetailWindow.h"
#include "SlateIM.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"

#if WITH_METADATA
#include "EditorCategoryUtils.h"
#endif

namespace AudioMotorModelDebugWidgetPrivate
{
	
	FString GenerateParamString(const FProperty& ParamProperty, void* ParamValuePtr)
	{
		check(ParamValuePtr);

		FString Message;
		FString ParamName = ParamProperty.GetName();
		
		if (const FIntProperty* IntProp = CastField<FIntProperty>(&ParamProperty))
		{
			const int32 Value = IntProp->GetPropertyValue(ParamValuePtr);
			Message = FString::Printf(TEXT("%s: %d"), *ParamName, Value);
		}
		else if (const FFloatProperty* FloatProp = CastField<FFloatProperty>(&ParamProperty))
		{
			const float Value = FloatProp->GetPropertyValue(ParamValuePtr);
			Message = FString::Printf(TEXT("%s: %f"), *ParamName, Value);
		}
		else if (const FBoolProperty* BoolProp = CastField<FBoolProperty>(&ParamProperty))
		{
			const bool Value = BoolProp->GetPropertyValue(ParamValuePtr);
			Message = FString::Printf(TEXT("%s: %d"), *ParamName, Value);
		}
		else
		{
			Message = FString::Printf(TEXT("Could not create param string for param %s"), *ParamName);
		}

		return Message;
	}

#if WITH_METADATA
	bool IsPropertyHiddenByClassCategory(const FProperty& InProperty, const UClass& ContainerClass)
	{
		FString PropertyCategory = InProperty.GetMetaData(TEXT("Category"));
		const UClass* ClassToCheck = &ContainerClass;

		while (ClassToCheck)
		{
			TArray<FString> HiddenCategoryList;
			FEditorCategoryUtils::GetClassHideCategories(ClassToCheck, HiddenCategoryList);
			
			for (FString& Category : HiddenCategoryList)
			{
				if (Category == PropertyCategory)
				{
					return true;
				}
			}

			ClassToCheck = ClassToCheck->GetSuperClass();
		}

		return false;
	}
#endif
}


FAudioMotorModelDebugWidget::FAudioMotorModelDebugWidget()
{
}

void FAudioMotorModelDebugWidget::Draw()
{
	UpdateDebugContexts();
	PollRegisteredModels();
	PollParameterGraphs();
	
	SlateIM::BeginBorder(FAppStyle::GetBrush("ToolPanel.GroupBorder"));
	SlateIM::Fill();
	SlateIM::HAlign(HAlign_Fill);
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::Padding({ 5.f });

	SlateIM::BeginScrollBox();
	{
		DrawMotorModelSelection();
		DrawParameterGraphs();
	}
	SlateIM::EndScrollBox();
	
	SlateIM::EndBorder();
}

void FAudioMotorModelDebugWidget::DrawMotorModelSelection()
{
	SlateIM::BeginScrollBox();
	{
		SlateIM::BeginVerticalStack();
		{
			SlateIM::HAlign(HAlign_Center);
			SlateIM::MaxHeight(150.f);
			SlateIM::BeginScrollBox();
			{
				for (int32 i = 0; i < RegisteredMotorModels.Num(); ++i)
				{
					if(const UAudioMotorModelComponent* Model = RegisteredMotorModels[i])
					{
						if(!IsValid(Model))
						{
							continue;
						}
				
						SlateIM::BeginHorizontalStack();
						{
							bool CheckState = SelectedMotorModelIndex == i;

							SlateIM::VAlign(VAlign_Center);
							SlateIM::CheckBox(TEXT(""), CheckState);
					
							if(CheckState && SelectedMotorModelIndex != i)
							{
								MotorSimDebugContexts.Empty();
								SelectedMotorModelIndex = i;
							}
					
							SlateIM::VAlign(VAlign_Center);

							const bool bHasValidOwner = Model->GetOwner() && Model->GetOwner()->IsValidLowLevel();
							const FString MotorModelOwnerName = bHasValidOwner ? Model->GetOwner()->GetName() : Model->GetName();
							const bool bIsBeingDriven = Model->GetCachedInputData().bDriving && Model->GetRuntimeInfo().Rpm > 0.f;
							SlateIM::Text(MotorModelOwnerName, bIsBeingDriven ? FColor::Green : FColor::Silver);
						}
						SlateIM::EndHorizontalStack();
					}
				}
			}
			SlateIM::EndScrollBox();
		}
		SlateIM::EndVerticalStack();
	}

	SlateIM::Spacer({ 1, 1 });

	if(RegisteredMotorModels.Num() > 0)
	{
		//Number of vehicles could change because of gameplay, so we need to check our index is still valid
		if(SelectedMotorModelIndex < 0 || SelectedMotorModelIndex > RegisteredMotorModels.Num() - 1)
		{
			MotorSimDebugContexts.Empty();
		}
		
		SelectedMotorModelIndex = FMath::Clamp(SelectedMotorModelIndex, 0, RegisteredMotorModels.Num() - 1);
		
		InstantiateMotorModelDetails(RegisteredMotorModels[SelectedMotorModelIndex]);
	}
	SlateIM::EndScrollBox();
}

void FAudioMotorModelDebugWidget::InstantiateMotorModelDetails(UAudioMotorModelComponent* Model)
{
	ensure(Model);
	AudioMotorModelDebug::FDebugContext* MotorModelDebugContext = CreateMotorModelObjectDebugContext(Model);
	
	if(!MotorModelDebugContext)
	{
		return;
	}
	
	MotorModelDebugContext->bDrawGraphs = true;
	
	SlateIM::BeginHorizontalStack();
	{
		DrawParameterSelection(*MotorModelDebugContext);
		CreateModelSimComponentsDebugContexts(Model);
		DrawMotorSimComponentList();
	}
	SlateIM::EndHorizontalStack();
}

void FAudioMotorModelDebugWidget::DrawObjectPropertiesTree(const TObjectPtr<UObject>& InObject)
{
	if(!ensureMsgf(InObject, TEXT("Expected valid object, property tree will nopt be drawn")))
	{
		return;	
	}
	
	if (SlateIM::BeginTableRowChildren())
	{
		for (TFieldIterator<FProperty> PropIt(InObject->GetClass()); PropIt; ++PropIt)
		{
			const FProperty* Property = *PropIt;

			if(!Property)
			{
				continue;
			}

			const bool bIsEditableProperty = Property->HasAllPropertyFlags(CPF_Edit) && !Property->HasAnyPropertyFlags(CPF_EditConst);
			const bool bUISupportedType = AudioMotorModelDebuggerPropertyUI::CanDrawProperty(*Property) || CastField<FArrayProperty>(Property);
			bool bHiddenByMetadata = false;
                                
#if WITH_METADATA
			bHiddenByMetadata = AudioMotorModelDebugWidgetPrivate::IsPropertyHiddenByClassCategory(*Property, *UAudioMotorSimComponent::StaticClass());								
#endif
								
			if (bIsEditableProperty && bUISupportedType && !bHiddenByMetadata)
			{
				if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					bool bSupportedArrayType = AudioMotorModelDebuggerPropertyUI::CanDrawProperty(*ArrayProperty->Inner);

					if(!bSupportedArrayType)
					{
						continue;
					}
										
					if (SlateIM::NextTableCell())
					{
						SlateIM::Text(ArrayProperty->GetName());
						if (SlateIM::BeginTableRowChildren())
						{
							constexpr bool bDisplayArrayName = false;
							AudioMotorModelDebuggerPropertyUI::DrawArrayPropertyUI(*ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(InObject.Get()), bDisplayArrayName);
						}
						SlateIM::EndTableRowChildren();
					}
				}
				else
				{
					if (SlateIM::NextTableCell())
					{
						AudioMotorModelDebuggerPropertyUI::DrawPropertyUI(*Property, InObject.Get());
					}
				}
			}
		}
	}
	SlateIM::EndTableRowChildren();
}

void FAudioMotorModelDebugWidget::DrawMotorSimComponentDetailButton(AudioMotorModelDebug::FDebugContext& DebugContext)
{
	if(!DebugContext.DebuggedObject)
	{
		return;
	}
	
	UAudioMotorSimComponent* DebuggedMotorSimComponent = Cast<UAudioMotorSimComponent>(DebugContext.DebuggedObject);

	if(!DebuggedMotorSimComponent)
	{
		return;
	}
	
	SlateIM::VAlign(VAlign_Center);
	if(SlateIM::Button(TEXT("Debug")))
	{
		if(TSharedPtr<FAudioMotorModelDebugWidgetChildWindow> DetailWindow = DebugContext.DetailWindow)
		{
			DetailWindow->ToggleWidget();
		}
		else
		{
			FString WindowName = FString::Printf(TEXT("%s (%s)"), *DebuggedMotorSimComponent->GetName(), *DebuggedMotorSimComponent->GetOuter()->GetName());
			DebugContext.DetailWindow = MakeShared<FMotorSimComponentDetailWindow>(WindowName);
			DebugContext.DetailWindow->ToggleWidget();
		}
	}
}

void FAudioMotorModelDebugWidget::DrawMotorSimComponentList()
{
	SlateIM::BeginVerticalStack();
	{
		SlateIM::Text(TEXT("Motor Sim Components"), FColorList::Green);
		
		const FTableRowStyle* TableRowStyle = &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow");
		SlateIM::BeginTable(nullptr, TableRowStyle);
		{
			SlateIM::AddTableColumn(TEXT("Components"), TEXT("Components"));
			{
				for(AudioMotorModelDebug::FDebugContext& DebugContext : MotorSimDebugContexts)
				{
					if(UAudioMotorSimComponent* DebuggedMotorSimComponent = Cast<UAudioMotorSimComponent>(DebugContext.DebuggedObject))
					{
						bool bIsMotorSimEnabled = DebuggedMotorSimComponent->bEnabled;

						if (SlateIM::NextTableCell())
						{
							//Draws motor sim name and a checkboxes to enable/disable it and print its params
							SlateIM::BeginHorizontalStack();
							{
								SlateIM::SetToolTip(TEXT("Is Enabled"));
								SlateIM::CheckBox(TEXT(""), bIsMotorSimEnabled);

								if(bIsMotorSimEnabled != DebuggedMotorSimComponent->bEnabled)
								{
									DebuggedMotorSimComponent->bEnabled = bIsMotorSimEnabled;
								}
								
								if(!bIsMotorSimEnabled)
								{
									//disable widgets related to this component
									SlateIM::BeginDisabledState();
								}
								
								SlateIM::SetToolTip(TEXT("Graph local contexts"));
								bool bDrawGraphCheckBox = DebugContext.bDrawGraphs;
								SlateIM::CheckBox(TEXT(""), bDrawGraphCheckBox);

								if(DebugContext.bDrawGraphs != bDrawGraphCheckBox)
								{
									if(DebugContext.bDrawGraphs)
									{
										DebugContext.bDrawGraphs = false;
									}
									else 
									{
										DebugContext.bDrawGraphs = true;;
									}
								}

								SlateIM::VAlign(VAlign_Center);
								FString ComponentName = FString::Printf(TEXT("%s"), *DebuggedMotorSimComponent->GetName());
								SlateIM::Text(ComponentName);
								
								DrawMotorSimComponentDetailButton(DebugContext);
							}
							SlateIM::EndHorizontalStack();
						}
						
						//Expandable tree showing properties
						DrawObjectPropertiesTree(DebugContext.DebuggedObject);

						if(!bIsMotorSimEnabled)
						{
							SlateIM::EndDisabledState();
						}
					}
				}
			}
		}
		SlateIM::EndTable();
	}
	SlateIM::EndVerticalStack();
}

void FAudioMotorModelDebugWidget::RegisterTrackedObject(const AudioMotorModelDebug::FDebugContext& InTrackedObject)
{
	MotorSimDebugContexts.AddUnique(InTrackedObject);
}

void FAudioMotorModelDebugWidget::UnregisterTrackedObject(TObjectPtr<UObject> DebuggedObject)
{
	if(!ensure(DebuggedObject))
	{
		return;
	}
	
	MotorSimDebugContexts.RemoveAllSwap([&](const AudioMotorModelDebug::FDebugContext& DebugContext)
	{
		return DebugContext.DebuggedObject == DebuggedObject;
	});
}

void FAudioMotorModelDebugWidget::GenerateParamSelector(const FProperty& ParamProperty, void* PropertyContainer)
{
	ensure(PropertyContainer);
	
	const bool bIsParamDrawn = PropertiesToGraph.Find(&ParamProperty) != INDEX_NONE;
	bool CheckState = bIsParamDrawn;
	
	SlateIM::BeginHorizontalStack();
	{
		SlateIM::VAlign(VAlign_Center);
		SlateIM::CheckBox(TEXT(""), CheckState);

		if(CheckState != bIsParamDrawn)
		{
			if(!bIsParamDrawn)
			{
				PropertiesToGraph.AddUnique(&ParamProperty);
			}
			else
			{
				PropertiesToGraph.RemoveAllSwap([PropToRemoveName = ParamProperty.GetFName()](const FProperty* ArrayProperty)
				{
					return ArrayProperty->GetFName() == PropToRemoveName;
				});
			}
		}

		void* ValuePtr = ParamProperty.ContainerPtrToValuePtr<void>(PropertyContainer);

		SlateIM::VAlign(VAlign_Center);
		SlateIM::Text(AudioMotorModelDebugWidgetPrivate::GenerateParamString(ParamProperty, ValuePtr));
	}
	SlateIM::EndHorizontalStack();		
}

TSharedPtr<AudioMotorModelDebug::IParamGraph> FAudioMotorModelDebugWidget::CreateParamGraph(const FProperty& ParamProperty, void* PropertyContainer)
{
	ensure(PropertyContainer);
	return MakeShared<AudioMotorModelDebug::FPropertyGraph>(ParamProperty.GetFName(), ParamProperty, PropertyContainer);
}

TSharedPtr<AudioMotorModelDebug::IParamGraph> FAudioMotorModelDebugWidget::CreateParamGraph(const FName& GraphedPropertyName, const FProperty& ParamProperty, void* PropertyContainer)
{
	ensure(PropertyContainer);
	return MakeShared<AudioMotorModelDebug::FPropertyGraph>(GraphedPropertyName, ParamProperty, PropertyContainer);
}

void FAudioMotorModelDebugWidget::RegisterMotorModelComponent(UAudioMotorModelComponent* Component)
{
	if(!ensureAlways(Component))
	{
		return;
	}
	
	RegisteredMotorModels.AddUnique(Component);
}

void FAudioMotorModelDebugWidget::SendAdditionalDebugData(UObject* Object, const FInstancedStruct& AdditionalData)
{
	if(!ensure(Object))
	{
		return;
	}

	if(!ensure(AdditionalData.IsValid()))
	{
		return;
	}
		
	for(AudioMotorModelDebug::FDebugContext& DebugContext : MotorSimDebugContexts)
	{
		if(DebugContext.DebuggedObject == Object)
		{
			auto FindDebugDataOfSameType = [&](const FInstancedStruct& ArrayDebugData)
			{
				return ArrayDebugData.GetScriptStruct() == AdditionalData.GetScriptStruct();
			};

			if(FInstancedStruct* MatchingDebugDataPtr = DebugContext.DebugData.FindByPredicate(FindDebugDataOfSameType))
			{
				*MatchingDebugDataPtr = AdditionalData;
			}
			else
			{
				DebugContext.DebugData.AddUnique(AdditionalData);
			}
		}
	}
}

void FAudioMotorModelDebugWidget::UpdateDebugContexts()
{
	for (int32 i = MotorSimDebugContexts.Num() - 1; i >= 0; --i)
	{
		AudioMotorModelDebug::FDebugContext& MotorSimDebugContext = MotorSimDebugContexts[i];
		
		if(!MotorSimDebugContext.DebuggedObject || !MotorSimDebugContext.DebuggedObject->IsValidLowLevel())
		{
			MotorSimDebugContexts.RemoveAt(i);
			continue;
		}
				
		if(const UAudioMotorModelComponent* AsMotorModelComponent = Cast<UAudioMotorModelComponent>(MotorSimDebugContext.DebuggedObject))
		{
			if(!MotorSimDebugContext.bDrawGraphs)
			{
				continue;
			}
			
			for(FInstancedStruct& DebugData : MotorSimDebugContext.DebugData)
			{
				if(FAudioMotorSimInputContext* AsInputContext = DebugData.GetMutablePtr<FAudioMotorSimInputContext>())
				{
					*AsInputContext = AsMotorModelComponent->GetCachedInputData();
				}
				else if (FAudioMotorSimRuntimeContext* AsRuntimeContext = DebugData.GetMutablePtr<FAudioMotorSimRuntimeContext>())
				{
					*AsRuntimeContext = AsMotorModelComponent->GetRuntimeInfo();
				}
			}
		}
#if WITH_EDITORONLY_DATA
		else if(const UAudioMotorSimComponent* AsMotorSimComponent = Cast<UAudioMotorSimComponent>(MotorSimDebugContext.DebuggedObject))
		{
			for(FInstancedStruct& DebugData : MotorSimDebugContext.DebugData)
			{
				if(MotorSimDebugContext.bDrawGraphs)
				{
					if(FAudioMotorSimInputContext* AsInputContext = DebugData.GetMutablePtr<FAudioMotorSimInputContext>())
					{
						*AsInputContext = AsMotorSimComponent->CachedInput;
					}
					else if (FAudioMotorSimRuntimeContext* AsRuntimeContext = DebugData.GetMutablePtr<FAudioMotorSimRuntimeContext>())
					{
						*AsRuntimeContext = AsMotorSimComponent->CachedRuntimeInfo;
					}
				}
			}
		}
#endif
		if(MotorSimDebugContext.DetailWindow)
		{
			MotorSimDebugContext.DetailWindow->SendDebugData(MotorSimDebugContext.DebugData);
		}
	}
}

void FAudioMotorModelDebugWidget::PollRegisteredModels()
{
	for(auto It = RegisteredMotorModels.CreateIterator(); It; ++It)
	{
		TObjectPtr<UAudioMotorModelComponent> MotorModel = *It;
		
		if(!MotorModel || !MotorModel->IsValidLowLevel())
		{
			It.RemoveCurrent();
		}
	}

	if(RegisteredMotorModels.IsEmpty())
	{
		MotorSimDebugContexts.Empty();
	}
}

void FAudioMotorModelDebugWidget::DrawParameterGraphs()
{
	SlateIM::BeginVerticalStack();
	{
		SlateIM::BeginVerticalStack();
		{
			SlateIM::Text(TEXT("Parameter Graphs"), FColorList::Green);

			SlateIM::BeginHorizontalStack();
			{
				SlateIM::Text(TEXT("Num Graphs in Row "));
				SlateIM::MinWidth(50.f);
				if (SlateIM::EditableText(GraphRowSizeText))
				{
					LiveGraphRowSize = FMath::Clamp(FCString::Atoi(*GraphRowSizeText), 1, MaxGraphRowSize);
				}
			}
			SlateIM::EndHorizontalStack();
		
			SlateIM::Text(FString::Printf(TEXT("Scale: %f"), ParamGraphScale));
			SlateIM::Slider(ParamGraphScale, 0, ParamGraphScaleMax, 1);
		
			SlateIM::Text(FString::Printf(TEXT("NumFrames: %0.f"), ParamGraphNumFrames));
			SlateIM::Slider(ParamGraphNumFrames, 1, ParamGraphNumFramesMax, 1);
		}
		SlateIM::EndVerticalStack();
	
		SlateIM::BeginVerticalStack();
		{
			int32 NumDrawnGraphs = 0;
			for(const FProperty* PropertyToGraph : PropertiesToGraph)
			{
				if(!PropertyToGraph)
				{
					continue;
				}
			
				for(AudioMotorModelDebug::FDebugContext& DebugContext : MotorSimDebugContexts)
				{
					if(!DebugContext.bDrawGraphs)
					{
						continue;
					}
		
					for(TPair<FName, TSharedPtr<AudioMotorModelDebug::IParamGraph>>& PropertyGraphPair : DebugContext.ParamGraphs)
					{
						if(PropertyToGraph->GetName() == PropertyGraphPair.Key && PropertyGraphPair.Value)
						{
							if(NumDrawnGraphs % LiveGraphRowSize == 0)
							{
								SlateIM::BeginHorizontalStack();
							}
							
							PropertyGraphPair.Value->Draw();
							PropertyGraphPair.Value->SetScale(ParamGraphScale);
							PropertyGraphPair.Value->SetNumFrames(ParamGraphNumFrames);
							NumDrawnGraphs++;

							if(NumDrawnGraphs % LiveGraphRowSize == 0)
							{
								SlateIM::EndHorizontalStack();
							}
						}
					}
				}
			}

			if(NumDrawnGraphs % LiveGraphRowSize != 0)
			{
				SlateIM::EndHorizontalStack();
			}
		}
		SlateIM::EndVerticalStack();
	}
	SlateIM::EndVerticalStack();

}

void FAudioMotorModelDebugWidget::PollParameterGraphs()
{
	for (auto It = PropertiesToGraph.CreateIterator(); It; ++It)
    {
        const FProperty* Property = *It;
    
		if(!ensure(Property))
		{
			It.RemoveCurrent();
			continue;
		}
		
		for(AudioMotorModelDebug::FDebugContext& DebugContext : MotorSimDebugContexts)
		{
			if(!DebugContext.ParamGraphs.Find(Property->GetFName()))
			{
				TSharedPtr<AudioMotorModelDebug::IParamGraph> ParamGraphPtr = nullptr;

				for(FInstancedStruct& DebugData : DebugContext.DebugData)
				{
					if(DebugData.GetScriptStruct()->FindPropertyByName(Property->GetFName()))
					{
						const FString GraphName = FString::Printf(TEXT("%s (%s)"), *Property->GetName(), DebugContext.DebuggedObject ? *DebugContext.DebuggedObject->GetName() : TEXT("NULL"));
						ParamGraphPtr = CreateParamGraph(FName(GraphName), *Property, DebugData.GetMutableMemory());
					}
				}

				if(ParamGraphPtr)
				{
					DebugContext.ParamGraphs.Add(Property->GetFName(), ParamGraphPtr);	
				}

			}
		}
	}

	//Remove properties that no longer need to be graphed
	for(AudioMotorModelDebug::FDebugContext& DebugContext : MotorSimDebugContexts)
	{
		if(PropertiesToGraph.IsEmpty())
		{
			DebugContext.ParamGraphs.Empty();
		}
		else
		{
			TArray<FName> PropertiesToRemove;
			
			for(const TPair<FName, TSharedPtr<AudioMotorModelDebug::IParamGraph>>& PropertyGraphPair : DebugContext.ParamGraphs)
			{
				auto FindMatchingParamGraph = [GraphedPropertyName = PropertyGraphPair.Key](const FProperty* ArrayProperty)
				{
					return ArrayProperty && GraphedPropertyName == ArrayProperty->GetFName();
				};
				
				if(PropertiesToGraph.FindByPredicate(FindMatchingParamGraph) == nullptr)
				{
					PropertiesToRemove.AddUnique(PropertyGraphPair.Key);
				}
			}
			
			for(const FName& PropertyToRemove : PropertiesToRemove)
			{
				DebugContext.ParamGraphs.Remove(PropertyToRemove);
			}
			
		}
	}
}

void FAudioMotorModelDebugWidget::DrawParameterSelection(AudioMotorModelDebug::FDebugContext& DebugContext)
{
	SlateIM::BeginHorizontalStack();
	{
		SlateIM::BeginScrollBox();
		{
			SlateIM::Text(TEXT("Runtime Context"), FColorList::Green);
			
			auto FindRuntimeContextData = [](const FInstancedStruct& StoredStruct)
			{
				return StoredStruct.GetPtr<FAudioMotorSimRuntimeContext>() != nullptr;
			};
			
			if(FInstancedStruct* RuntimeContext = DebugContext.DebugData.FindByPredicate(FindRuntimeContextData))
			{
				for (TFieldIterator<FProperty> PropIt(FAudioMotorSimRuntimeContext::StaticStruct()); PropIt; ++PropIt)
				{
					const FProperty* Property = *PropIt;

					if(!Property)
					{
						continue;
					}
				
					GenerateParamSelector(*Property, RuntimeContext->GetMutableMemory());
				}
			}
		}
		SlateIM::EndScrollBox();

		SlateIM::Spacer({ 5, 1 });

		SlateIM::BeginScrollBox();
		{
			SlateIM::Text(TEXT("Input Context"), FColorList::Green);

			auto FindInputContextData = [](const FInstancedStruct& StoredStruct)
			{
				return StoredStruct.GetPtr<FAudioMotorSimInputContext>() != nullptr;
			};
			
			if(FInstancedStruct* InputContext = DebugContext.DebugData.FindByPredicate(FindInputContextData))
			{
				for (TFieldIterator<FProperty> PropIt(FAudioMotorSimInputContext::StaticStruct()); PropIt; ++PropIt)
				{
					const FProperty* Property = *PropIt;

					if(!Property)
					{
						continue;
					}
				
					GenerateParamSelector(*Property, InputContext->GetMutableMemory());
				}
			}
		}
		SlateIM::EndScrollBox();
	}
	SlateIM::EndHorizontalStack();
}

AudioMotorModelDebug::FDebugContext* FAudioMotorModelDebugWidget::CreateMotorModelObjectDebugContext(UObject* ObjectToTrack)
{
	ensure(ObjectToTrack);

	uint32 Index = MotorSimDebugContexts.AddUnique(AudioMotorModelDebug::FDebugContext(ObjectToTrack));
	
	ensure(&MotorSimDebugContexts[Index]);

	MotorSimDebugContexts[Index].DebugData.Add(FInstancedStruct::Make<FAudioMotorSimInputContext>());
	MotorSimDebugContexts[Index].DebugData.Add(FInstancedStruct::Make<FAudioMotorSimRuntimeContext>());

	return &MotorSimDebugContexts[Index];
}

void FAudioMotorModelDebugWidget::CreateModelSimComponentsDebugContexts(const UAudioMotorModelComponent* Model)
{
	if(!ensure(Model))
	{
		return;
	}
	


	for(const FMotorSimEntry& MotorSimComponent : Model->SimComponents)
	{
		if(UObject* MotorSimObj = MotorSimComponent.Sim.GetObject())
		{
			CreateMotorModelObjectDebugContext(MotorSimObj);
		}
	}
}