// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMotorModelDebugWidgetChildWindow.h"
#include "AudioMotorModelDebuggerParamGraph.h"
#include "AudioMotorSimTypes.h"
#include "IAudioMotorSim.h"
#include "SlateIMWidgetBase.h"
#include "StructUtils/InstancedStruct.h"

struct FInstancedStruct;
class UAudioMotorModelComponent;

namespace AudioMotorModelDebug
{
	//Structs that hold information about an object to be interpreted/plotted by the debugger
	struct FDebugContext
	{
		FDebugContext() = default;
		
		explicit FDebugContext(const TObjectPtr<UObject>& InDebuggedObject)
			: DebuggedObject(InDebuggedObject)
		{}

		FDebugContext(const TObjectPtr<UObject>& InDebuggedObject, const TArray<FInstancedStruct>& InDebugData, const TMap<FName, TSharedPtr<IParamGraph>>& InParamGraphs, bool bInDrawGraphs)
			: DebuggedObject(InDebuggedObject)
			, DebugData(InDebugData)
			, ParamGraphs(InParamGraphs)
			, bDrawGraphs(bInDrawGraphs)
		{}

		~FDebugContext()
		{
			if(DetailWindow && DetailWindow->IsWidgetEnabled())
			{
				DetailWindow->DisableWidget();
			}
		}
				
		bool operator==(const FDebugContext& A) const
		{
			return DebuggedObject == A.DebuggedObject;
		}
		
		TObjectPtr<UObject> DebuggedObject;
		TArray<FInstancedStruct> DebugData;
		TMap<FName, TSharedPtr<IParamGraph>> ParamGraphs;
		TSharedPtr<FAudioMotorModelDebugDetailWindow> DetailWindow;
		bool bDrawGraphs = false;
	};
}

class FAudioMotorModelDebugWidget
{
public:
	FAudioMotorModelDebugWidget();
	void Draw();
	void RegisterMotorModelComponent(UAudioMotorModelComponent* Component);
	void SendAdditionalDebugData(UObject* Object, const FInstancedStruct& AdditionalData);

private:
	void UpdateDebugContexts();

	void PollRegisteredModels();
	void DrawMotorModelSelection();
	void DrawParameterSelection(AudioMotorModelDebug::FDebugContext& DebugContext);

	void InstantiateMotorModelDetails(UAudioMotorModelComponent* Model);
	void DrawMotorSimComponentList();
	AudioMotorModelDebug::FDebugContext* CreateMotorModelObjectDebugContext(UObject* ObjectToTrack);
	void CreateModelSimComponentsDebugContexts(const UAudioMotorModelComponent* Model);
	
	void GenerateParamSelector(const FProperty& ParamProperty, void* PropertyContainer);
	void DrawParameterGraphs();
	void PollParameterGraphs();
	
	void DrawObjectPropertiesTree(const TObjectPtr<UObject>& InObject);
	void DrawMotorSimComponentDetailButton(AudioMotorModelDebug::FDebugContext& DebugContext);

	TSharedPtr<AudioMotorModelDebug::IParamGraph> CreateParamGraph(const FProperty& ParamProperty, void* PropertyContainer);
	TSharedPtr<AudioMotorModelDebug::IParamGraph> CreateParamGraph(const FName& GraphedPropertyName, const FProperty& ParamProperty, void* PropertyContainer);
	
	void RegisterTrackedObject(const AudioMotorModelDebug::FDebugContext& InTrackedObject);
	void UnregisterTrackedObject(TObjectPtr<UObject> DebuggedObject);
	
	TArray<TObjectPtr<UAudioMotorModelComponent>> RegisteredMotorModels;
	
	int32 SelectedMotorModelIndex = 0;
	
	int32 GraphRowSize = 2;
	int32 MaxGraphRowSize = 50;
	int32 LiveGraphRowSize = GraphRowSize;
	FString GraphRowSizeText = FString::FromInt(GraphRowSize);
	
	float ParamGraphScale = 1.0f;         
	float ParamGraphScaleMax = 5.0f;
	
	float ParamGraphNumFrames = 300.0f;         
	float ParamGraphNumFramesMax = 1000.0f;        
	
	TArray<AudioMotorModelDebug::FDebugContext>	MotorSimDebugContexts;
	TArray<const FProperty*> PropertiesToGraph;
};
