// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateIM.h"
#include "SlateIMWidgetBase.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/ObjectKey.h"

class FAudioMotorModelDebugWidgetChildWindow : public FSlateIMWidgetBase
{
public:
	FAudioMotorModelDebugWidgetChildWindow(const FStringView& WindowTitle, FVector2f WindowSize)
		: FSlateIMWidgetBase(WindowTitle)
		, WindowTitle(WindowTitle)
		, WindowSize(WindowSize)
	{}

protected:
	virtual void DrawWindow(float DeltaTime) = 0;

private:
	virtual void DrawWidget(float DeltaTime) final override
	{
		if (!SlateIM::CanUpdateSlateIM())
    	{
    		return;
    	}
    	
    	const bool bIsDrawingWindow = SlateIM::BeginWindowRoot(GetWidgetName(), WindowTitle, WindowSize);
    	if (bIsDrawingWindow)
    	{
    		DrawWindow(DeltaTime);
    	}
    	SlateIM::EndRoot();
    
    	if (!bIsDrawingWindow)
    	{
    		DisableWidget();
    	}
	}
	
	FString WindowTitle;
	FVector2f WindowSize;
};


class FAudioMotorModelDebugDetailWindow : public FAudioMotorModelDebugWidgetChildWindow
{
public:
	FAudioMotorModelDebugDetailWindow(const FStringView& WindowTitle, const FVector2f& WindowSize)
		: FAudioMotorModelDebugWidgetChildWindow(WindowTitle, WindowSize)
	{}

	virtual void SendDebugData(TConstArrayView<FInstancedStruct> DebugDataView) = 0;

};