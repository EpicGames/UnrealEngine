// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMotorModelDebuggerParamGraph.h"

AudioMotorModelDebug::FDoubleRingBufferParamGraph::FDoubleRingBufferParamGraph(const FName& ParameterName, const FParamGraphSettings& InGraphSettings)
	: ParamName(ParameterName)
	, GraphRange(InGraphSettings.GraphRange)
	, GraphLineColor(InGraphSettings.GraphLineColor)
	, bIsAdaptive(InGraphSettings.bAdaptive)
{}

void AudioMotorModelDebug::FDoubleRingBufferParamGraph::DrawGraph(const double& ParamValue)
{
	if (GraphValues.Num() >= NumFrames)
	{
		GraphValues.PopFront();
	}
			
	GraphValues.Emplace(ParamValue);

	SlateIM::Padding(FMargin(1, 1));

	SlateIM::BeginVerticalStack();
	{
		SlateIM::BeginHorizontalStack();
		{
			SlateIM::Text(ParamName.ToString(), GraphLineColor);
			SlateIM::Text(FString::Printf(TEXT("%0.3lf"), ParamValue), GraphLineColor);
		}
		SlateIM::EndHorizontalStack();


		SlateIM::BeginHorizontalStack();

		SlateIM::MaxHeight(100.f * Scale);
		SlateIM::MaxWidth(200.f * Scale);

				
		SlateIM::BeginGraph();

		if(ParamValue > GraphRange.GetUpperBoundValue())
		{
			GraphRange.SetUpperBoundValue(ParamValue);
		}
				
		if(ParamValue < GraphRange.GetLowerBoundValue())
		{
			GraphRange.SetLowerBoundValue(ParamValue);
		}
				
		SlateIM::GraphLine(GraphValues.Compact(), GraphLineColor, 3.f, GraphRange);
		SlateIM::EndGraph();
				
		SlateIM::BeginVerticalStack();
		SlateIM::Padding(FMargin(0, 0));
		SlateIM::VAlign(VAlign_Top);
		SlateIM::Text(FString::Printf(TEXT("%0.3lf"), GraphRange.GetUpperBoundValue()), GraphLineColor);
		SlateIM::Fill();
		SlateIM::VAlign(VAlign_Bottom);
		SlateIM::Text(FString::Printf(TEXT("%0.3lf"), GraphRange.GetLowerBoundValue()), GraphLineColor);
		SlateIM::EndVerticalStack();

		SlateIM::EndHorizontalStack();

	}
	SlateIM::EndVerticalStack();
}

AudioMotorModelDebug::FPropertyGraph::FPropertyGraph(const FName& InPropertyName, const FProperty& InParamProperty, const void* PropertyContainerPtr, const FParamGraphSettings& InGraphSettings)
	: FDoubleRingBufferParamGraph(InPropertyName, InGraphSettings)
	, PropertyValuePtr(InParamProperty.ContainerPtrToValuePtr<void>(PropertyContainerPtr))
{
	if (const FIntProperty* IntProp = CastField<FIntProperty>(&InParamProperty))
	{
		PropertyFieldVariant.Set<const FIntProperty*>(IntProp);
	}
	else if (const FFloatProperty* FloatProp = CastField<FFloatProperty>(&InParamProperty))
	{
		PropertyFieldVariant.Set<const FFloatProperty*>(FloatProp);
	}
	else if (const FBoolProperty* BoolProp = CastField<FBoolProperty>(&InParamProperty))
	{
		PropertyFieldVariant.Set<const FBoolProperty*>(BoolProp);
	}
	else
	{
		ensureMsgf(false, TEXT("Unsupported property type for graphing"));
	}
}

void AudioMotorModelDebug::FPropertyGraph::Draw()
{
	const double PropertyValue = GetPropertyValue<double>();
	DrawGraph(PropertyValue);
	
	GraphValues.Emplace(PropertyValue);
}

