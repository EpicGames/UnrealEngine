// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/RingBuffer.h"
#include "IAudioMotorModelDebugger.h"
#include "Misc/TVariant.h"
#include "SlateIM.h"

namespace AudioMotorModelDebug
{
	struct FParamGraphSettings
	{
		FDoubleRange GraphRange = FDoubleRange(0, 1);
		FLinearColor GraphLineColor = FLinearColor::MakeRandomColor();
		bool bAdaptive = true;
	};
	
	class IParamGraph
	{
	public:

		virtual ~IParamGraph() = default;
		
		virtual void Draw() = 0;
		virtual void SetScale(const float InScale) = 0;
		virtual void SetNumFrames(const uint32 InNumFrames) = 0;
	};

	template <typename RingBufferType>
	class TParamGraphRingBuffer : public IParamGraph
	{
	public:
		virtual void Draw() override {};
		virtual void SetScale(const float InScale) override
		{
			Scale = InScale;
		}
		
		virtual void SetNumFrames(const uint32 InNumFrames) override
		{
			int32 ClampedInNumFrames = FMath::Clamp(InNumFrames, 1, NumFramesMax);
			
			if(ClampedInNumFrames < GraphValues.Num())
			{
				int32 FramesToPop = FMath::Abs(GraphValues.Num() - ClampedInNumFrames);
				GraphValues.Pop(FramesToPop);	
			}

			NumFrames = InNumFrames;
		};
	
	protected:
		float Scale = 1.f;
		int32 NumFrames = 100;
		uint32 NumFramesMax = 1000;
		
		TRingBuffer<RingBufferType> GraphValues;
		
	};
	
	class FDoubleRingBufferParamGraph : public TParamGraphRingBuffer<double>
	{
	public:
		FDoubleRingBufferParamGraph(const FName& ParameterName, const FParamGraphSettings& InGraphSettings = FParamGraphSettings());

	protected:
		void DrawGraph(const double& ParamValue);

		const FName ParamName;

		FDoubleRange GraphRange;
		FLinearColor GraphLineColor;
		
		bool bIsAdaptive = false;
	};

	
	//Param Graph that gets its value from a FProperty in the provided container ptr, WHICH IS NOT kept alive by the graph
	class FPropertyGraph : public FDoubleRingBufferParamGraph
	{
	public:
		FPropertyGraph(const FName& InPropertyName, const FProperty& InParamProperty, const void* PropertyContainerPtr, const FParamGraphSettings& InGraphSettings = FParamGraphSettings());

		virtual void Draw() override;

	private:
		template <typename OutValue>
		OutValue GetPropertyValue()
		{
			if(!ensureMsgf(PropertyValuePtr, TEXT("Null property value ptr, property will not be graphed")))
			{
				return OutValue();
			}
			
			if (const FIntProperty** StoredIntPropPtr = PropertyFieldVariant.TryGet<const FIntProperty*>())
			{
				if(const FIntProperty* IntPropPtr = *StoredIntPropPtr)
				{
					return static_cast<OutValue>(IntPropPtr->GetPropertyValue(PropertyValuePtr));
				}
			}
			else if (const FFloatProperty** StoredFloatPropPtr = PropertyFieldVariant.TryGet<const FFloatProperty*>())
			{
				if(const FFloatProperty* FloatPropPtr = *StoredFloatPropPtr)
				{
					return static_cast<OutValue>(FloatPropPtr->GetPropertyValue(PropertyValuePtr));
				}
			}
			else if (const FBoolProperty** StoredBoolPropPtr = PropertyFieldVariant.TryGet<const FBoolProperty*>())
			{
				if(const FBoolProperty* BoolPropPtr = *StoredBoolPropPtr)
				{
					return static_cast<OutValue>(BoolPropPtr->GetPropertyValue(PropertyValuePtr));
				}
			}
			
			return OutValue();
		}
		
		const void* PropertyValuePtr;
		TVariant<FEmptyVariantState, const FIntProperty*, const FFloatProperty*, const FBoolProperty*> PropertyFieldVariant;
	};

	//Param Graph that gets its value from a ParamType*, WITHOUT keeping it alive
	template <typename ParamType>
	class FParamPtrGraph : public FDoubleRingBufferParamGraph
	{
	public:
		
		FParamPtrGraph(const FName& InParamName, const ParamType* InParamToDraw, const FParamGraphSettings& InGraphSettings = FParamGraphSettings())
			: FDoubleRingBufferParamGraph(InParamName, InGraphSettings)
			, ParamToDraw(InParamToDraw)
		{}

		virtual void Draw() override
		{
			if(ParamToDraw == nullptr)
			{
				return;
			}

			DrawGraph(*ParamToDraw);
		}
			
	private:
		ParamType* ParamToDraw;
	};
}

