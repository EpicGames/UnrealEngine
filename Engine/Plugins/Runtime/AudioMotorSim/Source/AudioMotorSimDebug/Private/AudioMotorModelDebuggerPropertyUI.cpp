// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMotorModelDebuggerPropertyUI.h"

#include "SlateIM.h"

namespace AudioMotorModelDebuggerPropertyUIPrivate
{
	constexpr float MinSpinboxValue = 0.f;
	constexpr float MaxSpinboxValue = 20000.f;
}

void AudioMotorModelDebuggerPropertyUI::DrawPropertyUI(const FProperty& InProperty, void* ContainerPtr, bool bDisplayPropertyName)
{
	if(!ensure(ContainerPtr))
	{
		return;
	}

	if(!CanDrawProperty(InProperty))
	{
		return;
	}

	SlateIM::BeginHorizontalStack();
	{
		if(bDisplayPropertyName)
		{
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(FString::Printf(TEXT("%s"), *InProperty.GetName()));
		}
		
		if (const FFloatProperty* FloatProp = CastField<FFloatProperty>(&InProperty))
		{
			const float PropertyValue = FloatProp->GetPropertyValue_InContainer(ContainerPtr);
			float SpinBoxValue = PropertyValue;

			float MinSpinBoxValue = AudioMotorModelDebuggerPropertyUIPrivate::MinSpinboxValue;
			float MaxSpinBoxValue = AudioMotorModelDebuggerPropertyUIPrivate::MaxSpinboxValue;

#if WITH_METADATA
			TPair<float,float> MinMax = GetMinMaxClampFromPropertyMetadata<float>(InProperty, TPair<float, float>(MinSpinBoxValue, MaxSpinBoxValue));
			MinSpinBoxValue = MinMax.Key;
			MaxSpinBoxValue = MinMax.Value;
#endif

			SlateIM::VAlign(VAlign_Center);
			if (SlateIM::SpinBox(SpinBoxValue, MinSpinBoxValue, MaxSpinBoxValue))
			{
				if(SpinBoxValue != PropertyValue)
				{
					FloatProp->SetValue_InContainer(ContainerPtr, SpinBoxValue);
				}
			}
		}
		else if (const FIntProperty* IntProp = CastField<FIntProperty>(&InProperty))
		{
			const int32 PropertyValue = IntProp->GetPropertyValue_InContainer(ContainerPtr);
			int32 SpinBoxValue = PropertyValue;

			int32 MinSpinBoxValue = AudioMotorModelDebuggerPropertyUIPrivate::MinSpinboxValue;
			int32 MaxSpinBoxValue = AudioMotorModelDebuggerPropertyUIPrivate::MaxSpinboxValue;

#if WITH_EDITORONLY_DATA
			TPair<int32,int32> MinMax = GetMinMaxClampFromPropertyMetadata<int32>(InProperty, TPair<int32, int32>(MinSpinBoxValue, MaxSpinBoxValue));
			MinSpinBoxValue = MinMax.Key;
			MaxSpinBoxValue = MinMax.Value;
#endif

			SlateIM::VAlign(VAlign_Center);
			if (SlateIM::SpinBox(SpinBoxValue, MinSpinBoxValue, MaxSpinBoxValue))
			{
				if(SpinBoxValue != PropertyValue)
				{
					IntProp->SetValue_InContainer(ContainerPtr, SpinBoxValue);
				}
			}
		}
		else if (const FBoolProperty* BoolProp = CastField<FBoolProperty>(&InProperty))
		{
			const bool PropertyValue = BoolProp->GetPropertyValue_InContainer(ContainerPtr);
			bool CheckBoxValue = PropertyValue;

			SlateIM::VAlign(VAlign_Center);
			if (SlateIM::CheckBox(TEXT(""), CheckBoxValue))
			{
				if(CheckBoxValue != PropertyValue)
				{
					BoolProp->SetValue_InContainer(ContainerPtr, &CheckBoxValue);
				}
			}
			
		}
	}
	SlateIM::EndHorizontalStack();
}

void AudioMotorModelDebuggerPropertyUI::DrawArrayPropertyUI(const FArrayProperty& InArrayProperty, const void* ContainerPtr, bool bDisplayArrayName)
{
	if(bDisplayArrayName)
	{
		SlateIM::Text(InArrayProperty.GetName());
	}
	
	FScriptArrayHelper ArrayHelper(&InArrayProperty, ContainerPtr);

	SlateIM::BeginVerticalStack();
	for (int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ++ElementIndex)
	{
		void* ValueAddress = ArrayHelper.GetRawPtr(ElementIndex);
		
		SlateIM::BeginHorizontalStack();
		{
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(FString::Printf(TEXT("%d"), ElementIndex));
			DrawPropertyUI(*InArrayProperty.Inner, ValueAddress, false);
		}
		SlateIM::EndHorizontalStack();
	}
	SlateIM::EndVerticalStack();
}

bool AudioMotorModelDebuggerPropertyUI::CanDrawProperty(const FProperty& InProperty)
{
	bool bSupportedType = InProperty.IsA<FFloatProperty>() || InProperty.IsA<FIntProperty>() || InProperty.IsA<FBoolProperty>();
	return bSupportedType;

}

