// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/Tuple.h"
#include "UObject/UnrealType.h"

namespace AudioMotorModelDebuggerPropertyUI
{
	void DrawPropertyUI(const FProperty& InProperty, void* ContainerPtr, bool bDisplayPropertyName = true);
	void DrawArrayPropertyUI(const FArrayProperty& InArrayProperty, const void* ContainerPtr, bool bDisplayArrayName = true);
	bool CanDrawProperty(const FProperty& InProperty);

#if WITH_METADATA
	template<typename NumericType>
	TPair<NumericType, NumericType> GetMinMaxClampFromPropertyMetadata(const FProperty& TargetProperty, TPair<NumericType, NumericType> MinMaxLimits = TPair<NumericType, NumericType>(TNumericLimits<NumericType>::Lowest(), TNumericLimits<NumericType>::Max()))
	{
		static_assert(TIsArithmetic<NumericType>::Value, "Property to clamp must have numeric type");

		const FString& MetaClampMinString = TargetProperty.GetMetaData("ClampMin");
		const FString& MetaClampMaxString = TargetProperty.GetMetaData("ClampMax");
		const FString& MetaUIMinString = TargetProperty.GetMetaData("UIMin");
		const FString& MetaUIMaxString = TargetProperty.GetMetaData("UIMax");
				
		NumericType ClampMin = MinMaxLimits.Key;
		if (!MetaClampMinString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(ClampMin, *MetaClampMinString);
		}

		NumericType ClampMax = MinMaxLimits.Value;
		if (!MetaClampMaxString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(ClampMax, *MetaClampMaxString);
		}
		
		NumericType UIMin = MinMaxLimits.Key;
		if (!MetaUIMinString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(UIMin, *MetaUIMinString);
		}
		
		NumericType UIMax = MinMaxLimits.Value;
		if (!MetaUIMaxString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(UIMax, *MetaUIMaxString);
		}

		const NumericType MinAllowedValue = FMath::Max(UIMin, ClampMin);
		const NumericType MaxAllowedValue = FMath::Min(UIMax, ClampMax); 
		
		return TPair<NumericType, NumericType>(MinAllowedValue, MaxAllowedValue);
	}
#endif
	
}


