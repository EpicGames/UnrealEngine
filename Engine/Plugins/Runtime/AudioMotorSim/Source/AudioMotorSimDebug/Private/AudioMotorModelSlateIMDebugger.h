// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMotorModelComponent.h"
#include "AudioMotorModelSlateIMWindow.h"
#include "IAudioMotorModelDebugger.h"

class FAudioMotorModelSlateIMDebugger : public IAudioMotorModelDebugger
{
public:
	FAudioMotorModelSlateIMDebugger();

	virtual void RegisterComponentWithDebugger(UAudioMotorModelComponent* MotorModelComponent) override;
	virtual void SendAdditionalDebugData(UObject* Object, const FInstancedStruct& InData) override;

private:
	FAudioMotorModelSlateIMWindow SlateIMWindow;
	
#if WITH_ENGINE                                                                      
	FAudioMotorModelSlateIMViewportWidget SlateIMViewportWidget;
#endif
};