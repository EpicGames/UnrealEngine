// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMotorModelSlateIMWindow.h"

#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Textures/SlateIcon.h"
#include "SlateIM.h"


namespace AudioMotorModelSlateIMWindowPrivate
{
	FVector2f WindowSize(850, 900);
	FVector2f ViewportAlignment = FVector2f(0.0f, 0);
	FAnchors ViewportAnchors = FAnchors(0.0f, 0);
}

FAudioMotorModelSlateIMWindow::FAudioMotorModelSlateIMWindow(const TCHAR* Command, const TCHAR* CommandHelp)
	: FSlateIMNomadTabBase(TEXT("Audio Motor Model Slate IM Window"), Command, CommandHelp, FSlateIcon("CoreStyle", "Launcher.Tabs.Tools"))
{}

void FAudioMotorModelSlateIMWindow::DrawContent(float DeltaTime)
{
	DebugWidget.Draw();
}


FAudioMotorModelSlateIMViewportWidget::FAudioMotorModelSlateIMViewportWidget(const TCHAR* Command, const TCHAR* CommandHelp)
	: FSlateIMWidgetWithCommandBase(Command, CommandHelp)
{                                                                                
	Layout.Anchors = AudioMotorModelSlateIMWindowPrivate::ViewportAnchors;
	Layout.Alignment = AudioMotorModelSlateIMWindowPrivate::ViewportAlignment;
}

void FAudioMotorModelSlateIMViewportWidget::DrawWidget(float DeltaTime)
{
	if (GEngine && GEngine->GameViewport)
	{		
		if (SlateIM::BeginViewportRoot("AudioMotorModelSlateIMViewportWidget", GEngine->GameViewport, Layout))
		{
			DebugWidget.Draw();
		}
		SlateIM::EndRoot();
	}
}