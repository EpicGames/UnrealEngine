// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMotorModelDebugWidget.h"
#include "MotorSimComponentDetailWindow.h"
#include "SlateIMWidgetBase.h"

class FAudioMotorModelSlateIMWindow : public FSlateIMNomadTabBase
{
public:
	FAudioMotorModelSlateIMWindow(const TCHAR* Command, const TCHAR* CommandHelp);

	virtual void DrawContent(float DeltaTime) override;

	FAudioMotorModelDebugWidget DebugWidget;
};


class FAudioMotorModelSlateIMViewportWidget : public FSlateIMWidgetWithCommandBase              
{                                                                                    
public:                                                                              
	FAudioMotorModelSlateIMViewportWidget(const TCHAR* Command, const TCHAR* CommandHelp);

	virtual void DrawWidget(float DeltaTime) override;

	FAudioMotorModelDebugWidget DebugWidget;                                                   

private:                                                                             
	SlateIM::FViewportRootLayout Layout;                                             
};                                                                                   