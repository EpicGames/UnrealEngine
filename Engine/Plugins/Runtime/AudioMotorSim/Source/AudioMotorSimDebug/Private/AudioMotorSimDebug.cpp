// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMotorSimDebug.h"

#include "AudioMotorModelSlateIMDebugger.h"
#include "Features/IModularFeatures.h"
#include "IAudioMotorModelDebugger.h"

#define LOCTEXT_NAMESPACE "FAudioMotorSimDebugModule"

void FAudioMotorSimDebugModule::StartupModule()
{
	SlateIMDebugger = MakeUnique<FAudioMotorModelSlateIMDebugger>();
	IModularFeatures::Get().RegisterModularFeature(AudioMotorModelDebugger::DebuggerModularFeatureName, SlateIMDebugger.Get());
}

void FAudioMotorSimDebugModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(AudioMotorModelDebugger::DebuggerModularFeatureName, SlateIMDebugger.Get());
	SlateIMDebugger.Reset();
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FAudioMotorSimDebugModule, AudioMotorSimDebug)