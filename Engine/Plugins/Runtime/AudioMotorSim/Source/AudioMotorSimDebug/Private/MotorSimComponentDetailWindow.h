// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMotorModelDebuggerParamGraph.h"
#include "AudioMotorModelDebugWidgetChildWindow.h"

namespace AudioMotorModelDebugger
{
	struct FDebugData;
}

class FMotorSimComponentDetailWindow : public FAudioMotorModelDebugDetailWindow
{
public:
	explicit FMotorSimComponentDetailWindow(const FStringView& WindowTitle);
	
	virtual void SendDebugData(TConstArrayView<FInstancedStruct> DebugDataView) override;

private:
	virtual void DrawWindow(float DeltaTime) override;
	void GenerateParamSelector(const TPair<FName, double>& ParamNameValue);
	void DrawParameterSelection();
	void DrawParameterGraphs();
	void EnableMotorSimComponentDetails() const;

	TMap<FName, double> ParameterValues;
	TMap<FName, AudioMotorModelDebug::FParamPtrGraph<const double>> ParamGraphs;

	int32 GraphRowSize = 2;
	int32 MaxGraphRowSize = 50;
	int32 LiveGraphRowSize = GraphRowSize;
	FString GraphRowSizeText = FString::FromInt(GraphRowSize);
	
	float ParamGraphScale = 1.0f;         
	float ParamGraphScaleMax = 5.0f;
	
	float ParamGraphNumFrames = 300.0f;         
	float ParamGraphNumFramesMax = 1000.0f;
};
