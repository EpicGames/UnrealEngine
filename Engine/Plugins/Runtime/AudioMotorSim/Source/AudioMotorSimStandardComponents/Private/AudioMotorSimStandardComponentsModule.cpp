// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FAudioMotorSimStandardComponentsModule : public IModuleInterface
{

};

IMPLEMENT_MODULE(FAudioMotorSimStandardComponentsModule, AudioMotorSimStandardComponents);
