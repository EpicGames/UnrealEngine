// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoostMotorSimComponent.h"
#include "AudioMotorSimTypes.h"
#include "Kismet/KismetMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoostMotorSimComponent)

void UBoostMotorSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	if(bModifyPitch)
	{
		const float PitchTarget = BoostToPitchCurve.GetRichCurveConst()->Eval(Input.Boost);
		RuntimeInfo.Pitch = FMath::FInterpTo(RuntimeInfo.Pitch, PitchTarget, Input.DeltaTime, PitchModifierInterpSpeed);
	}

	if (Input.Boost <= 0.f)
	{
		ActiveTime = 0.0f;
		Super::Update(Input, RuntimeInfo);
		return;
	}

	ActiveTime += Input.DeltaTime;

	float InterpScale = ThrottleScale;
	if(ScaleThrottleWithBoostStrength)
	{
		InterpScale *= Input.Boost;
	}
	
	Input.Throttle = UKismetMathLibrary::Ease(InterpScale, 1.0, FMath::Min(ActiveTime / InterpTime, 1.0f), EEasingFunc::EaseIn, InterpExp);

	Super::Update(Input, RuntimeInfo);
}
void UBoostMotorSimComponent::Reset()
{
	Super::Reset();
	ActiveTime = 0.f;
}
