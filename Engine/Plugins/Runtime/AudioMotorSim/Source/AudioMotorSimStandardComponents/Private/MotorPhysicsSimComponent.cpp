// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotorPhysicsSimComponent.h"
#include "AudioMotorSimLogs.h"
#include "AudioMotorSimTypes.h"
#include "StandardAudioMotorSimConfigDataTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotorPhysicsSimComponent)

void UMotorPhysicsSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	if(ensure(Weight > 0.f && EngineGearRatio > 0.f) == false)
	{
		return;
	}
		
	const int32 MaxGear = GearRatios.Num() - 1;
	const float GearRatio = GetGearRatio(RuntimeInfo.Gear, Input.bClutchEngaged);
	const float Throttle = FMath::Abs(Input.Throttle);
	
	int32& Gear = RuntimeInfo.Gear;
	const int32 PrevGear = Gear;
	
	const float ClutchModifier = Input.bClutchEngaged ? ClutchedForceModifier : 1.f;
	const float ClutchDecelModifier = Input.bClutchEngaged ? FMath::Max(ClutchedDecelScale * (1.f - Throttle), 0.f) : 1.f;

	//Forces in Newtons.
	const float CurrentEngineTorque = EngineTorque * Throttle * GearRatio * ClutchModifier;
	
	const float BrakingForce = Input.Brake * BrakingHorsePower;
	const float WindResistance = WindResistancePerVelocity * FMath::Square(RuntimeInfo.VirtualSpeedKmh);	
	const float GroundFrictionResistance = Weight * GroundFriction * Input.SurfaceFrictionModifier;
	const float EngineResistance = EngineTorque * FMath::Max(Input.MotorFrictionModifier - Throttle, 0.0f) * EngineFriction * ClutchDecelModifier;
	
	const float NetForce = CurrentEngineTorque - BrakingForce - WindResistance - GroundFrictionResistance - EngineResistance;

	UE_LOGF(LogAudioMotorSim, Verbose, "[UMotorPhysicsSimComponent] NetForce: %f (CurrentEngineTorque: %f - BrakingForce: %f - WindResistance %f - GroundFrictionResistance %f - EngineResistance %f", NetForce, CurrentEngineTorque, BrakingForce, WindResistance, GroundFrictionResistance, EngineResistance)

	// Convert Rpm to velocity to apply forces
	float VirtualSpeedKmh =  CalcVelocity(GearRatio, RuntimeInfo.Rpm);
	VirtualSpeedKmh += ((NetForce / Weight) * Input.DeltaTime);
	VirtualSpeedKmh = FMath::Max(VirtualSpeedKmh, 0.0f);
	RuntimeInfo.VirtualSpeedKmh = VirtualSpeedKmh;

	// Now working backwards, what is the engine RPM as a results of this velocity and gear ratio.
	float Rpm = CalcRpm(GearRatio, VirtualSpeedKmh);
	
	UE_LOGF(LogAudioMotorSim, Verbose, "[UMotorPhysicsSimComponent] Curr Gear %d - RPM: %f", Gear, Rpm);
	UE_LOGF(LogAudioMotorSim, Verbose, "[UMotorPhysicsSimComponent] Prev Gear %d - RPM: %f", (Gear >= 1 ? Gear - 1 : 0), CalcRpm(GetGearRatio((Gear >= 1 ? Gear - 1 : 0), Input.bClutchEngaged), VirtualSpeedKmh));
	
	// Now determine if we need to shift.
	if (Input.bCanShift && (Rpm > UpShiftMaxRpm))
	{
		if (Gear < MaxGear || bUseInfiniteGears)
		{
			++Gear;
		}

		Rpm = CalcRpm(GetGearRatio(Gear, Input.bClutchEngaged), VirtualSpeedKmh);
	}
	// If the RPM that we'd be at if we downshift is below DownShiftRPM, then it makes sense to downshift.
	else if (Gear >= 1 && DownShiftStartRpm > CalcRpm(GetGearRatio(Gear - 1, Input.bClutchEngaged), VirtualSpeedKmh))
	{
		Gear = bAlwaysDownshiftToZerothGear ? 0 : Gear - 1;
		Rpm = DownShiftStartRpm;
	}

	if(Gear != PrevGear)
	{
		OnGearChangedEvent.Broadcast(Gear);
	}

	RuntimeInfo.Rpm = FMath::FInterpTo(RuntimeInfo.Rpm, Rpm, Input.DeltaTime, RpmInterpSpeed);

#if !UE_BUILD_SHIPPING
	if(ShouldCollectDebugData())
	{
		if(!CachedDebugData.IsValid())
		{
			CachedDebugData.InitializeAs<FAudioMotorSimDebugDataBase>();;
		}
		
		if(FAudioMotorSimDebugDataBase* DebugData = CachedDebugData.GetMutablePtr<FAudioMotorSimDebugDataBase>())
		{
			DebugData->ParameterValues.Add("CurrentEngineTorque", CurrentEngineTorque);
			DebugData->ParameterValues.Add("BrakingForce", BrakingForce);
			DebugData->ParameterValues.Add("WindResistance", WindResistance);
			DebugData->ParameterValues.Add("GroundFrictionResistance", GroundFrictionResistance);
			DebugData->ParameterValues.Add("EngineResistance", EngineResistance);
			DebugData->ParameterValues.Add("NetForce", NetForce);
			DebugData->ParameterValues.Add("VirtualSpeedKmh", VirtualSpeedKmh);
			DebugData->ParameterValues.Add("ClutchDecelModifier", ClutchDecelModifier);
			DebugData->ParameterValues.Add("Gear", Gear);
			DebugData->ParameterValues.Add("GearRatio", GetGearRatio(Gear, Input.bClutchEngaged));
			DebugData->ParameterValues.Add("RPM", GetGearRatio(Gear, Input.bClutchEngaged));

		}
	}
#endif

	Super::Update(Input, RuntimeInfo);
}

void UMotorPhysicsSimComponent::ConfigMotorSim(const FInstancedStruct& InConfigData)
{
	Super::ConfigMotorSim(InConfigData);

	if(const FMotorPhysicsSimConfigData* MotorPhysicsSimConfigData = InConfigData.GetPtr<FMotorPhysicsSimConfigData>())
	{
		Weight = MotorPhysicsSimConfigData->Weight;
		EngineTorque = MotorPhysicsSimConfigData->EngineTorque;
		BrakingHorsePower = MotorPhysicsSimConfigData->BrakingHorsePower;
	}
}

float UMotorPhysicsSimComponent::CalcRpm(const float InGearRatio, const float InSpeed) const
{
	return InSpeed * InGearRatio / EngineGearRatio;
}

float UMotorPhysicsSimComponent::CalcVelocity(const float InGearRatio, const float InRpm) const
{
	check(InGearRatio > 0.f);
	return InRpm * EngineGearRatio / InGearRatio;
}

float UMotorPhysicsSimComponent::GetGearRatio(const int32 InGear, const bool bInClutched) const
{
	if(bInClutched)
	{
		return ClutchedGearRatio;
	}
	
	int32 Gear = InGear;
	if(bUseInfiniteGears == false)
	{
		Gear = FMath::Clamp(Gear, 0, GearRatios.Num());
	}
	
	return InterpGearRatio(Gear);
}

float UMotorPhysicsSimComponent::InterpGearRatio(const int32 InGear) const
{
	const int32 MaxGear = GearRatios.Num() - 1;
	if (InGear <= MaxGear)
	{
		return GearRatios[InGear];
	}

	return GearRatios[MaxGear] * FMath::Pow(InfiniteGearRatio, InGear - MaxGear);
}
