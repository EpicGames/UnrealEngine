// Copyright Epic Games, Inc. All Rights Reserved.

#include "ResistanceMotorSimComponent.h"
#include "AudioMotorSimTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResistanceMotorSimComponent)

void UResistanceMotorSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	if(Input.bClutchEngaged || !Input.bDriving || !Input.bGrounded)
	{
		return;
	}
	
	if (Input.Speed > MinSpeed)
	{
		check(MinSpeed != 0.f);

		const float SideFriction = SideSpeedFrictionCurve.GetRichCurveConst()->Eval(Input.SideSpeed);
		const float UpSpeedRatio = Input.UpSpeed / Input.Speed;
		const float ZFriction = UpSpeedMaxFriction * UpSpeedRatio;
		
		Input.SurfaceFrictionModifier += ZFriction + SideFriction;
		
#if !UE_BUILD_SHIPPING
		if(ShouldCollectDebugData())
		{
			if(!CachedDebugData.IsValid())
			{
				CachedDebugData.InitializeAs<FAudioMotorSimDebugDataBase>();;
			}
		
			if(FAudioMotorSimDebugDataBase* DebugData = CachedDebugData.GetMutablePtr<FAudioMotorSimDebugDataBase>())
			{
				DebugData->ParameterValues.Add("SideSpeed", Input.SideSpeed);
				DebugData->ParameterValues.Add("UpSpeed", Input.UpSpeed);
				DebugData->ParameterValues.Add("Speed", Input.Speed);
				DebugData->ParameterValues.Add("UpSpeed Ratio (UpSpeed / Speed)", UpSpeedRatio);
				DebugData->ParameterValues.Add("ZFriction", ZFriction);
				DebugData->ParameterValues.Add("SideFriction", SideFriction);
			}
		}
#endif
	}

	Super::Update(Input, RuntimeInfo);
}
