// Copyright Epic Games, Inc. All Rights Reserved.

#include "RevLimiterMotorSimComponent.h"
#include "AudioMotorSimTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RevLimiterMotorSimComponent)

void URevLimiterMotorSimComponent::RevLimitToTarget(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo, const float LimitRPM, const float TargetRPM)
{
	if (RuntimeInfo.Rpm >= LimitRPM && !bActive)
	{
		bActive = true;
		TimeRemaining = LimitTime;
		OnRevLimiterHit.Broadcast();
		OnRevLimiterStateChanged.Broadcast(bActive);
	}

	if(RuntimeInfo.Rpm <= TargetRPM && bActive)
	{
		bActive = false;
		OnRevLimiterStateChanged.Broadcast(bActive);

	}
		
	Input.Throttle = bActive ? 0.0f : Input.Throttle;
}

void URevLimiterMotorSimComponent::EngageClutchAndShiftBackToZero(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) const
{
	Input.bClutchEngaged = true;
	if(bShiftBackToZero)
	{
		RuntimeInfo.Gear = 0;
	}
}

void URevLimiterMotorSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	if(bRevLimitOnClutchEngaged && Input.bClutchEngaged)
	{
		RuntimeInfo.Gear = bShiftBackToZero ? 0 : RuntimeInfo.Gear;
		RevLimitToTarget(Input, RuntimeInfo, LimiterMaxRpm, ClutchedRecoverRPM);
		Super::Update(Input, RuntimeInfo);
		return;
	}

	const bool bIsGroundedDriving = Input.bDriving && Input.bGrounded;
	
	if(bRevLimitOnReverse && bIsGroundedDriving)
	{
		const bool bIsReversing = Input.ForwardSpeed < 0.f && Input.Throttle <= 0.f;

		if(bIsReversing)
		{
			if(bHoldRPMInReverse)
			{
				//Force RPM only if throttle is actually being held down
				if(Input.Throttle < 0 && RuntimeInfo.Rpm > ReverseMaxRPM)
				{
					RuntimeInfo.Rpm = ReverseMaxRPM;
				}
			}
			else
			{
				RevLimitToTarget(Input, RuntimeInfo, ReverseMaxRPM, ReverseRecoverRPM);
			}

			Super::Update(Input, RuntimeInfo);
			return;
		}
	}
	
	if (bIsGroundedDriving && Input.SideSpeed < SideSpeedThreshold)
	{
		TimeRemaining = 0.f;
		TimeInAir = 0.0f;

		if(bActive)
		{
			bActive = false;
			OnRevLimiterStateChanged.Broadcast(bActive);
		}

		Super::Update(Input, RuntimeInfo);
		return;
	}

	if(bActive == false)
	{
		bActive = true;
		OnRevLimiterStateChanged.Broadcast(bActive);
	}
	
	Input.bCanShift = false;
	
	// We've hit the limiter
	if (RuntimeInfo.Rpm >= LimiterMaxRpm)
	{
		TimeRemaining = LimitTime;
		RuntimeInfo.Rpm = LimiterMaxRpm;				
		OnRevLimiterHit.Broadcast();
	}

	if (TimeRemaining > 0.0f)
	{
		Input.Throttle = 0.0f;

		TimeRemaining -= Input.DeltaTime;
		
		EngageClutchAndShiftBackToZero(Input, RuntimeInfo);
	}
	else if (Input.bDriving == false)
	{
		Input.bClutchEngaged = true;
	}
	
	if (Input.bGrounded)
	{
		TimeInAir = 0.0f;
		Super::Update(Input, RuntimeInfo);
		return;
	}

	EngageClutchAndShiftBackToZero(Input, RuntimeInfo);
	
	if (TimeRemaining > 0.0f)
	{
		TimeInAir += Input.DeltaTime;
	}

	if (TimeInAir >= AirMaxThrottleTime)
	{
		Input.Throttle = 0.0f;
	}
	
#if !UE_BUILD_SHIPPING
	if(ShouldCollectDebugData())
	{
		if(!CachedDebugData.IsValid())
		{
			CachedDebugData.InitializeAs<FAudioMotorSimDebugDataBase>();;
		}
		
		if(FAudioMotorSimDebugDataBase* DebugData = CachedDebugData.GetMutablePtr<FAudioMotorSimDebugDataBase>())
		{
			DebugData->ParameterValues.Add("TimeRemaining", TimeRemaining);
			DebugData->ParameterValues.Add("TimeInAir", TimeInAir);
			DebugData->ParameterValues.Add("bActive", bActive);
			DebugData->ParameterValues.Add("bRevLimitOnClutchEngaged", bRevLimitOnClutchEngaged);
		}
	}
#endif

	Super::Update(Input, RuntimeInfo);
}

void URevLimiterMotorSimComponent::Reset()
{
	Super::Reset();

	TimeRemaining = 0.f;
	TimeInAir = 0.f;
}

