// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReverseMotorSimComponent.h"
#include "AudioMotorSimTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReverseMotorSimComponent)

void UReverseMotorSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	if (Input.ForwardSpeed >= 0.f)
	{
		Super::Update(Input, RuntimeInfo);
		return;
	}

	Input.MotorFrictionModifier *= ReverseEngineResistanceModifier;

	Super::Update(Input, RuntimeInfo);
}

