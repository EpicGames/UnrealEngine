// Copyright Epic Games, Inc. All Rights Reserved.

#include "RpmCurveMotorSimComponent.h"
#include "AudioMotorSimTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RpmCurveMotorSimComponent)

void URpmCurveMotorSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	if (Gears.Num() == 0)
	{
		Super::Update(Input, RuntimeInfo);
		return;
	}

	const float SpeedKmh = Input.Speed;
	
	int32 DesiredGear = 0;

	if (Input.bCanShift)
	{
		DesiredGear = GetDesiredGearForSpeed(SpeedKmh);
	}
	else
	{
		DesiredGear = RuntimeInfo.Gear;
	}

	const float SpeedCeil = Gears[DesiredGear].SpeedTopThreshold;
	const float SpeedFloor = DesiredGear > 0 ? Gears[DesiredGear - 1].SpeedTopThreshold : 0.0f;
	
	if (DesiredGear != RuntimeInfo.Gear)
	{
		if (DesiredGear > RuntimeInfo.Gear)
		{
			OnUpShift.Broadcast(DesiredGear);
		}
		else
		{
			OnDownShift.Broadcast(DesiredGear);
		}
		
		RuntimeInfo.Gear = DesiredGear;
	}
	
	const float SpeedRatio = FMath::Clamp(FMath::GetRangePct(SpeedFloor, SpeedCeil, SpeedKmh), 0.0f, 1.0f);
	const float TargetRpm = Gears[RuntimeInfo.Gear].RpmCurve.GetRichCurveConst()->Eval(SpeedRatio);

	if (InterpSpeed > 0.f)
	{
		RuntimeInfo.Rpm = FMath::FInterpTo(RuntimeInfo.Rpm, TargetRpm, Input.DeltaTime, InterpSpeed);
	}
	else
	{
		RuntimeInfo.Rpm = TargetRpm;
	}

	Super::Update(Input, RuntimeInfo);
}

int32 URpmCurveMotorSimComponent::GetDesiredGearForSpeed(const float Speed) const
{
	if (Gears.IsEmpty())
	{
		return 0;
	}
	
	for (int32 Gear = 0; Gear < Gears.Num(); ++Gear)
	{
		if (Speed < Gears[Gear].SpeedTopThreshold)
		{
			return Gear;
		}
	}

	return Gears.Num() - 1;
}

