// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThrottleStateMotorSimComponent.h"
#include "AudioMotorSimTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ThrottleStateMotorSimComponent)

void UThrottleStateMotorSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	const bool bCarIdling = Input.Throttle == 0.f;
    const bool bCarReversing = Input.Throttle < 0.f;
    const bool bFullThrottle = Input.Throttle >= 1.f;
    const bool bCarAccelerating = !bCarIdling && !bCarReversing;

    if (bFullThrottle)
    {
    	FullThrottleTime += Input.DeltaTime;
    }
    else if (FullThrottleTime > 0.0f)
    {
    	if (Input.Throttle <= 0.0f && FullThrottleTime > BlowoffMinThrottleTime)
    	{
    		OnEngineBlowoff.Broadcast(FullThrottleTime);
    		FullThrottleTime = 0.0f;
    	}

    	FullThrottleTime = FMath::Max(FullThrottleTime - Input.DeltaTime, 0.0f);
    }

    if (!bPrevCarAccelerating && bCarAccelerating)
    {
    	OnThrottleEngaged.Broadcast();
    }
    else if (!bPrevCarIdling && bCarIdling)
    {
    	OnThrottleReleased.Broadcast();
    }

    bPrevCarAccelerating = bCarAccelerating;
    bPrevCarIdling = bCarIdling;
	
	Super::Update(Input, RuntimeInfo);
}

void UThrottleStateMotorSimComponent::Reset()
{
	Super::Reset();

	FullThrottleTime = 0.f;

	bPrevCarAccelerating = false;
	bPrevCarIdling = true;
}
