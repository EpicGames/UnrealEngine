// Copyright Epic Games, Inc. All Rights Reserved.
#include "VelocitySyncMotorSimComponent.h"
#include "AudioMotorSimTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VelocitySyncMotorSimComponent)

void UVelocitySyncMotorSimComponent::Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo)
{
	if(Input.bClutchEngaged || !Input.bDriving || !Input.bGrounded)
	{
		Super::Update(Input, RuntimeInfo);
		return;
	}
	
	if (FMath::IsNearlyZero(Input.Throttle)
	   || (RuntimeInfo.Gear == 0 && FMath::Abs(Input.Throttle) < FirstGearThrottleThreshold))
	{
		NoThrottleTimeElapsed += Input.DeltaTime;
	}
	else
	{
		NoThrottleTimeElapsed = 0.0f;
	}

	if (NoThrottleTimeElapsed > NoThrottleTime)
	{
		InterpTimeLeft = InterpTime;
	}

	if (Input.Speed < SpeedThreshold || InterpTimeLeft > 0.f)
	{
		const float TargetRpm = SpeedToRpmCurve.GetRichCurveConst()->Eval(Input.Speed);
		RuntimeInfo.Rpm = FMath::FInterpTo(RuntimeInfo.Rpm, TargetRpm, Input.DeltaTime, InterpSpeed);
		
		InterpTimeLeft -= Input.DeltaTime;
	}

	Super::Update(Input, RuntimeInfo);
}
void UVelocitySyncMotorSimComponent::Reset()
{
	Super::Reset();

	NoThrottleTimeElapsed = 0.f;
	InterpTimeLeft = 0.f;
}
