// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"
#include "Curves/CurveFloat.h"
#include "BoostMotorSimComponent.generated.h"

// Uses Boost input to momentarily scale throttle input and pitch output
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class UBoostMotorSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()
public:

	// Scale the engine torque by this value when boost is active
	UPROPERTY(EditAnywhere, Category = "Throttle", meta = (ClampMin = "0.1"))
	float ThrottleScale = 5.f;

	// controls shape of the scaling
	UPROPERTY(EditAnywhere, Category = "Throttle", meta = (ClampMin = "0.1"))
	float InterpExp = 4.f;

	// How fast the torque scales when starting to boost
	UPROPERTY(EditAnywhere, Category = "Throttle", meta = (ClampMin = "0.1"))
	float InterpTime = 2.f;

	// whether to use the boost strength to scale ThrottleScale, or just check if it is > 0 to apply the throttle scalar
	UPROPERTY(EditAnywhere, Category = "Throttle", meta = (ClampMin = "0.1"))
	bool ScaleThrottleWithBoostStrength = false;

	// whether scale the overall pitch by the boost strength
	UPROPERTY(EditAnywhere, Category = "Pitch")
	bool bModifyPitch = true;
	
	// Speed at which pitch approaches its target value
	UPROPERTY(EditAnywhere, Category = "Pitch", meta = (EditCondition = "bModifyPitch = true"))
	float PitchModifierInterpSpeed = 1.f;

	// Curve to derive final pitch value (in playback speed) from boost strength
	UPROPERTY(EditAnywhere, Category = "Pitch", meta = (EditCondition = "bModifyPitch = true"))
	FRuntimeFloatCurve BoostToPitchCurve;

	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;
	virtual void Reset() override;
private:

	float ActiveTime = 0.f;
};
