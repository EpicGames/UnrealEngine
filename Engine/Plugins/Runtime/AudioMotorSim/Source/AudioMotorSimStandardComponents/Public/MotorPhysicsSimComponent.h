// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"

#include "MotorPhysicsSimComponent.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnGearChanged, int32, NewGear);

// Uses throttle input to run a physics simulation to drive RPM and shift gears when needed
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class UMotorPhysicsSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()

public:
	//The weight of the vehicle, used to calculate the generated RPM. Heavier weight means a slower rpm increase due to weight resistance and ground friction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Force", meta=(ClampMin = "0.0"))
	float Weight = 900.f;

	//The force of the engine. Large values will accelerate the vehicle more quickly, but also apply more friction when engine braking
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Force", meta=(ClampMin = "0.0"))
	float EngineTorque = 2500.f;

	//The force applied when braking. Large values will slow down the vehicle more quickly
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Force", meta=(ClampMin = "0.0"))
	float BrakingHorsePower = 6000.f;

	//How each gear responds to speed, expressed as a ratio. Higher number means a larger RPM range within that gear and quicker progression through the gear
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears")
	TArray<float> GearRatios = {3.5f, 2.f, 1.4f, 1.f, .7f};

	//The gear ratio to apply when clutched
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears", meta=(ClampMin = "0.0"))
	float ClutchedGearRatio = 10.f;

	//If true, the model will keep upshifting past the last defined gear. Gear ratio will be defined by the InfiniteGearRatio param
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears")
	bool bUseInfiniteGears = false;

	//If enabled, will always downshift to 0th gear when a downshift occurs
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears")
	bool bAlwaysDownshiftToZerothGear = false;

	// how much to scale gear ratio per gear past the max gear
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears", meta=(ClampMin = "0.0"))
	float InfiniteGearRatio = 0.9f;

	//The rpm at which an upshift occurs
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears", meta=(ClampMin = "0.0", ClampMax=1.0))
	float UpShiftMaxRpm = 0.97f;

	//The rpm at which a downshift occurs
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears", meta=(ClampMin = "0.0", ClampMax=1.0))
	float DownShiftStartRpm = 0.94f;

	// Engine Torque multiplier applied when clutched
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance", meta=(ClampMin = "0.0"))
	float ClutchedForceModifier = 1.f;

	// How much more quickly the engine decelerates off-throttle while clutched
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance", meta=(ClampMin = "0.0"))
	float ClutchedDecelScale = 1.f;

	//The main gear ratio of the engine. Corresponds to a RPM value of 1 if the vehicle is going at this speed.
	//Speed * GearRatio / EngineGearRatio = RPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance", meta=(ClampMin = "0.0"))
	float EngineGearRatio = 50.f;

	// How much of the torque is lost due to engine friction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance", meta=(ClampMin = "0.0"))
	float EngineFriction = 0.66f;

	// Coefficient of Rolling Resistance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance", meta=(ClampMin = "0.0"))
	float GroundFriction = 1.f;

	//Multiplier to apply to the velocity value to calculate wind resistance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance", meta=(ClampMin = "0.0"))
	float WindResistancePerVelocity = 0.015f;

	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PhysicsSim", meta=(ClampMin = "0.0"))
	float ThrottleInterpolationTime = 0.050f;

	//Interpolation value to when changing RPM 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "PhysicsSim",  meta=(ClampMin = "0.0"))
	float RpmInterpSpeed = 0.0f;

	UPROPERTY(BlueprintAssignable, Category = "PhysicsSim")
	FOnGearChanged OnGearChangedEvent;

	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;

	virtual void ConfigMotorSim(const FInstancedStruct& InConfigData) override;

private:
	float CalcRpm(const float InGearRatio, const float InSpeed) const;

	float CalcVelocity(const float InGearRatio, const float InRpm) const;

	float GetGearRatio(const int32 InGear, const bool bInClutched) const;

	float InterpGearRatio(const int32 InGear) const;
};
