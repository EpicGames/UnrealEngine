// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"
#include "Curves/CurveFloat.h"
#include "ResistanceMotorSimComponent.generated.h"

// Applys additional surface friction based on the angle of the vehicle's velocity
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class UResistanceMotorSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()
public:

	// How much to increase surface friction when driving straight up. Scales linearly based on driving angle.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance")
	float UpSpeedMaxFriction = 1.f;

	// Minimum speed to apply this extra resistance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance", meta = (ClampMin = "1.0"))
	float MinSpeed = 100.f;

	// Additional friction to add based on lateral speed
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Resistance")
	FRuntimeFloatCurve SideSpeedFrictionCurve;
	
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;
};
