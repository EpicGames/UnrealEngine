// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"

#include "RevLimiterMotorSimComponent.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnRevLimiterHit);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnRevLimiterStateChanged, bool, bNewState);

// Temporarily cuts throttle and reduces RPM when drifting or in the air
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class URevLimiterMotorSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()
public:
	//How long will the throttle be cut for when the limiter kicks in  
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter", Meta=(ClampMin="0.0", ClampMax="5.0"))
	float LimitTime = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter", Meta=(EditCondition=false, EditConditionHides, DeprecatedProperty, DeprecationMessage="DecelScale is deprecated."))
	float DecelScale = 0.f;

	//How long should the rev limiter be enabled for while in air 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter", Meta=(ClampMin="0.0", ClampMax="10.0"))
	float AirMaxThrottleTime = 0.f;

	//The side speed needed to trigger the rev limiter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter")
	float SideSpeedThreshold = 0.f;

	//Ceiling RPM for the Rev Limiter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter", Meta=(ClampMin="0.0", ClampMax="1.0"))
	float LimiterMaxRpm = 0.f;

	//If the car should shift back to 0 when the rev limiter is engaged
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter")
	bool bShiftBackToZero = false;

	//If enabled, rev limiter behavior will be triggered by having the clutch engaged
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter|Clutch")
	bool bRevLimitOnClutchEngaged = false;

	//When rev limiting on clutch, the RPM at which the throttle will be re-enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter|Clutch", Meta=(ClampMin="0.0", ClampMax="1.0", EditCondition="bRevLimitOnClutchEngaged"))
	float ClutchedRecoverRPM = 0.f;

	//If enabled, rev limiter behavior will be triggered when in reverse
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter|Reverse")
	bool bRevLimitOnReverse = false;

	//The max RPM allowed when reversing
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter|Reverse", Meta=(ClampMin="0.0", ClampMax="1.0", EditCondition="bRevLimitOnReverse", EditConditionHides))
	float ReverseMaxRPM = 0.f;
	
	//When rev limiting on reverse, the RPM at which the throttle will be re-enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter|Reverse", Meta=(ClampMin="0.0", ClampMax="1.0", EditCondition="bRevLimitOnReverse && !bHoldRPMInReverse", EditConditionHides))
	float ReverseRecoverRPM = 0.f;

	//Bypasses the limiter in reverse and holds the RPM at ReverseMaxRPM
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RevLimiter|Reverse", Meta=(EditCondition="bRevLimitOnReverse", EditConditionHides))
	bool bHoldRPMInReverse = false;

	UPROPERTY(BlueprintAssignable)
	FOnRevLimiterHit OnRevLimiterHit;

	UPROPERTY(BlueprintAssignable)
	FOnRevLimiterStateChanged OnRevLimiterStateChanged;

private:
	void RevLimitToTarget(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo, const float LimitRPM, const float TargetRPM);
	void EngageClutchAndShiftBackToZero(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) const;

	// Time remaining where the limiter is forcing throttle down
	float TimeRemaining = 0.f;
	float TimeInAir = 0.f;

	bool bActive = false;

public:
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;
	virtual void Reset() override;
};
