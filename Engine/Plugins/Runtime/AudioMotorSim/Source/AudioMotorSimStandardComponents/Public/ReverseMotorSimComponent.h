// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"
#include "ReverseMotorSimComponent.generated.h"

// Scales engine friction when reversing
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class UReverseMotorSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()
	
public:
	// How much to scale engine friction by when reversing
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Reverse", Meta=(ClampMin="0.0"))
	float ReverseEngineResistanceModifier = 1.f;
	
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;
};
