// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"
#include "Curves/CurveFloat.h"
#include "RpmCurveMotorSimComponent.generated.h"

USTRUCT(BlueprintType)
struct FMotorSimGearCurve
{
	GENERATED_BODY()

	// normalized curve, 0-1, of the output RPM from the last gear's top speed to this one's
	UPROPERTY(EditAnywhere, Category = "Gears")
	FRuntimeFloatCurve RpmCurve;

	// Speed at which the next gear starts
	UPROPERTY(EditAnywhere, Category = "Gears")
	float SpeedTopThreshold = 0.f;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnGearChangedEvent, int32, NewGear);

// Derives Gear and RPM directly from speed using hand drawn curves and gear thresholds
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class URpmCurveMotorSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()
public:
	
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;
	
	UPROPERTY(EditAnywhere, Category = "Gears")
	TArray<FMotorSimGearCurve> Gears;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gears", Meta=(ClampMin="0.0"))
	float InterpSpeed = 0.f;

	UPROPERTY(BlueprintAssignable)
	FOnGearChangedEvent OnUpShift;
	
	UPROPERTY(BlueprintAssignable)
	FOnGearChangedEvent OnDownShift;

private:
	int32 GetDesiredGearForSpeed(const float Speed) const;
};
