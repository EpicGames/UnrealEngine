// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMotorSimConfigData.h"

#include "StandardAudioMotorSimConfigDataTypes.generated.h"


USTRUCT(BlueprintType)
struct FMotorPhysicsSimConfigData : public FAudioMotorSimConfigData
{
	GENERATED_BODY()

	// The weight of the vehicle, used to calculate the generated RPM. Heavier weight means a slower rpm increase due to weight resistance and ground friction
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Force")
	float Weight = 900.f;

	// The force of the engine. Large values will accelerate the vehicle more quickly, but also apply more friction when engine braking
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Force")
	float EngineTorque = 2500.f;

	// The force applied when braking. Large values will slow down the vehicle more quickly
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Force")
	float BrakingHorsePower = 6000.f;
};
