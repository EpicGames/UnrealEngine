// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"
#include "ThrottleStateMotorSimComponent.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnEngineBlowoff, float, BlowoffStrength);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnThrottleChanged);

// Provides events for when the throttle changes state
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class UThrottleStateMotorSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()
	
public:
	UPROPERTY(BlueprintAssignable, Category = "ThrottleState")
	FOnThrottleChanged OnThrottleEngaged;
	
	UPROPERTY(BlueprintAssignable, Category = "ThrottleState")
	FOnThrottleChanged OnThrottleReleased;

	// Fires when the throttle is released, keeping track of how long the throttle was held for
	UPROPERTY(BlueprintAssignable, Category = "ThrottleState")
	FOnEngineBlowoff OnEngineBlowoff;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ThrottleState", Meta=(ClampMin="0.0"))
	float BlowoffMinThrottleTime = 1.f;
	
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;
	virtual void Reset() override;

private:
	float FullThrottleTime = 0.f;

	bool bPrevCarAccelerating = false;
	bool bPrevCarIdling = true;
};
