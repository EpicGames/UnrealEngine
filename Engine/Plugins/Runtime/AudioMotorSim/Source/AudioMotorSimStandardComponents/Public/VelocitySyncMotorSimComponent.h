// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAudioMotorSim.h"
#include "Curves/CurveFloat.h"
#include "VelocitySyncMotorSimComponent.generated.h"

// Sets Rpm directly from speed using a curve, if under a speed threshold or if the throttle is released for a period of time
UCLASS(ClassGroup = "AudioMotorSim", meta = (BlueprintSpawnableComponent))
class UVelocitySyncMotorSimComponent : public UAudioMotorSimComponent
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VelocitySync", Meta=(ClampMin="0.0"))
	float NoThrottleTime = 0.1f;

	// Speed below which gears will be ignored, and RPM will sync directly to velocity
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VelocitySync", Meta=(ClampMin="0.0"))
	float SpeedThreshold = 200.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VelocitySync")
	FRuntimeFloatCurve SpeedToRpmCurve;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VelocitySync", Meta=(ClampMin="0.0"))
	float InterpSpeed = 3.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VelocitySync", Meta=(ClampMin="0.0"))
	float InterpTime = 0.1f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VelocitySync", Meta=(ClampMin="0.0"))
	float FirstGearThrottleThreshold = 0.5f;
	
	virtual void Update(FAudioMotorSimInputContext& Input, FAudioMotorSimRuntimeContext& RuntimeInfo) override;
	virtual void Reset() override;

private:
	float NoThrottleTimeElapsed = 0.f;
	float InterpTimeLeft = 0.f;
};
